{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "OpenCV",
	"Description": "Plugin initializing OpenCV library to be used in engine.",
	"Category": "Computer Vision",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "PythonFoundationPackages",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		}
	],
	"Modules": [
		{
			"Name": "OpenCVHelper",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	],
	"PythonRequirements": [
		{
			"Platform": "All",
			"Requirements": [
				"opencv-python==4.5.5.62 --hash=sha256:c463d2276d8662b972d20ca9644702188507de200ca5405b89e1fe71c5c99989 --hash=sha256:130cc75d56b29aa3c5de8b6ac438242dd2574ba6eaa8bccdffdcfd6b78632f7f --hash=sha256:ac92e743e22681f30001942d78512c1e39bce53dbffc504e5645fdc45c0f2c47 --hash=sha256:2601388def0d6b957cc30dd88f8ff74a5651ae6940dd9e488241608cfa2b15c7"
			]
		}
	]
}