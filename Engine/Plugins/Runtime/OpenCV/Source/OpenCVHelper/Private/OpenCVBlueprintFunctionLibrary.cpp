// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenCVBlueprintFunctionLibrary.h"
#include "OpenCVHelper.h"

#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"

#if WITH_OPENCV
// IWYU pragma: begin_keep
#include "PreOpenCVHeaders.h"
#include "opencv2/aruco.hpp"
#include "opencv2/calib3d.hpp"
#include "opencv2/imgproc.hpp"
#include "PostOpenCVHeaders.h"
// IWYU pragma: end_keep
#endif	// WITH_OPENCV

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenCVBlueprintFunctionLibrary)

DEFINE_LOG_CATEGORY(LogOpenCVBlueprintFunctionLibrary);

int32 UOpenCVBlueprintFunctionLibrary::OpenCVChessboardDetectCorners(const UTextureRenderTarget2D* InRenderTarget, const FIntPoint InPatternSize, const bool bDebugDrawCorners, UTexture2D*& OutDebugTexture, TArray<FVector2D>& OutDetectedCorners)
{
#if WITH_OPENCV
	if (InRenderTarget == nullptr)
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVChessboardDetectCorners: Invalid InRenderTarget");
		return 0;
	}

	if (InRenderTarget->RenderTargetFormat != ETextureRenderTargetFormat::RTF_RGBA8)
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVChessboardDetectCorners: Invalid InRenderTarget texture format - only RGBA8 is currently supported");
		return 0;
	}

	FRenderTarget* RenderTarget = ((UTextureRenderTarget2D*)InRenderTarget)->GameThread_GetRenderTargetResource();
	if (RenderTarget == nullptr)
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVChessboardDetectCorners: Can't get render target resource from InRenderTarget");
		return 0;
	}

	// Copy the pixels from the RenderTarget
	TArray<FColor> Pixels;
	if (!RenderTarget->ReadPixels(Pixels))
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVChessboardDetectCorners: Failed to read pixels from RenderTarget");
		return 0;
	}

	// Check for invalid dimensions
	const FIntPoint Size = RenderTarget->GetSizeXY();
	if ((Size.X <= 0) || (Size.Y <= 0) || (Pixels.Num() != (Size.X * Size.Y)))
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVChessboardDetectCorners: Invalid number of pixels read from RenderTarget");
		return 0;
	}

	// Create OpenCV Mat with those pixels and convert to grayscale for image processing
	cv::Mat CvFrameColor(cv::Size(Size.X, Size.Y), CV_8UC4, Pixels.GetData());
	cv::Mat CvFrameGray;
	cv::cvtColor(CvFrameColor, CvFrameGray, cv::COLOR_RGBA2GRAY);

	// Setup the incoming pattern size and flags
	const cv::Size PatternSize(InPatternSize.X, InPatternSize.Y);
	const int32 Flags = cv::CALIB_CB_ADAPTIVE_THRESH | cv::CALIB_CB_NORMALIZE_IMAGE | cv::CALIB_CB_FAST_CHECK;
	std::vector<cv::Point2f> Corners;

	// Detect the chessboard in the image
	bool bAllCornersFound = cv::findChessboardCorners(CvFrameGray, PatternSize, Corners, Flags);
	const int32 NumCornersFound = Corners.size();

	// Draw debug info onto the frame if requested even if we don't find all the corners
	if (bDebugDrawCorners)
	{
		cv::drawChessboardCorners(CvFrameColor, PatternSize, Corners, bAllCornersFound);
		OutDebugTexture = FOpenCVHelper::TextureFromCvMat(CvFrameColor);
	}

	if (bAllCornersFound)
	{
		// Not completely sure why WinSize is set to these specific values, but every OpenCV example that I could find online just used these values
		// ZeroZone of (-1, -1) means there is no dead zone in the search kernel
		cv::Size WinSize(11, 11);
		cv::Size ZeroZone(-1, -1);
		cv::TermCriteria TermCriteria(cv::TermCriteria::Type::COUNT | cv::TermCriteria::Type::EPS, 30, 0.001);

		// Refine the corner locations
		cv::cornerSubPix(CvFrameGray, Corners, WinSize, ZeroZone, TermCriteria);

		OutDetectedCorners.Reserve(NumCornersFound);

		if (!Corners.empty())
		{
			for (const cv::Point2f& Corner : Corners)
			{
				OutDetectedCorners.Add(FVector2D(Corner.x, Corner.y));
			}
		}
	}

	return NumCornersFound;
#else
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVChessboardDetectCorners: Requires OpenCV");
		return 0;
	}
#endif	// WITH_OPENCV
}

int32 UOpenCVBlueprintFunctionLibrary::OpenCVArucoDetectMarkers(const UTextureRenderTarget2D* InRenderTarget, const EOpenCVArucoDictionary InDictionary, const EOpenCVArucoDictionarySize InDictionarySize, const bool bDebugDrawMarkers, const bool bEstimatePose, const float InMarkerLengthInMeters, const FOpenCVLensDistortionParametersBase& InLensDistortionParameters, UTexture2D*& OutDebugTexture, TArray<FOpenCVArucoDetectedMarker>& OutDetectedMarkers)
{
#if WITH_OPENCV
	if (InRenderTarget == nullptr)
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVArucoDetectMarkers: Invalid InRenderTarget");
		return 0;
	}

	if (bEstimatePose)
	{
		if (InMarkerLengthInMeters <= 0.0)
		{
			UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVArucoDetectMarkers: bEstimatePose specified, but InMarkerLengthInMeters is <= 0");
			return 0;
		}
	}

	if (InRenderTarget->RenderTargetFormat != ETextureRenderTargetFormat::RTF_RGBA8)
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVArucoDetectMarkers: Invalid InRenderTarget texture format - only RGBA8 is currently supported");
		return 0;
	}

	FRenderTarget* RenderTarget = ((UTextureRenderTarget2D*)InRenderTarget)->GameThread_GetRenderTargetResource();
	if (RenderTarget == nullptr)
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVArucoDetectMarkers: Can't get render target resource from InRenderTarget");
		return 0;
	}

	// Copy the pixels from the RenderTarget
	TArray<FColor> Pixels;
	if (!RenderTarget->ReadPixels(Pixels))
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVArucoDetectMarkers: Failed to read pixels from RenderTarget");
		return 0;
	}

	// Check for invalid dimensions
	const FIntPoint Size = RenderTarget->GetSizeXY();
	if ((Size.X <= 0) || (Size.Y <= 0) || (Pixels.Num() != (Size.X * Size.Y)))
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVArucoDetectMarkers: Invalid number of pixels read from RenderTarget");
		return 0;
	}

	// Create OpenCV Mat with those pixels and convert to grayscale for image processing
	cv::Mat CvFrameColor(cv::Size(Size.X, Size.Y), CV_8UC4, Pixels.GetData());
	cv::Mat CvFrameGray;
	cv::cvtColor(CvFrameColor, CvFrameGray, cv::COLOR_RGBA2GRAY);

	// Select the requested marker dictionary
	cv::aruco::PREDEFINED_DICTIONARY_NAME DictionaryName = cv::aruco::DICT_ARUCO_ORIGINAL;
	switch (InDictionary)
	{
		case EOpenCVArucoDictionary::Dict4x4:		switch (InDictionarySize)
													{
														case EOpenCVArucoDictionarySize::DictSize50:	DictionaryName = cv::aruco::DICT_4X4_50; break;
														case EOpenCVArucoDictionarySize::DictSize100:	DictionaryName = cv::aruco::DICT_4X4_100; break;
														case EOpenCVArucoDictionarySize::DictSize250:	DictionaryName = cv::aruco::DICT_4X4_250; break;
														case EOpenCVArucoDictionarySize::DictSize1000:	DictionaryName = cv::aruco::DICT_4X4_1000; break;
													}
													break;
		case EOpenCVArucoDictionary::Dict5x5:		switch (InDictionarySize)
													{
														case EOpenCVArucoDictionarySize::DictSize50:	DictionaryName = cv::aruco::DICT_5X5_50; break;
														case EOpenCVArucoDictionarySize::DictSize100:	DictionaryName = cv::aruco::DICT_5X5_100; break;
														case EOpenCVArucoDictionarySize::DictSize250:	DictionaryName = cv::aruco::DICT_5X5_250; break;
														case EOpenCVArucoDictionarySize::DictSize1000:	DictionaryName = cv::aruco::DICT_5X5_1000; break;
													}
													break;
		case EOpenCVArucoDictionary::Dict6x6:		switch (InDictionarySize)
													{
														case EOpenCVArucoDictionarySize::DictSize50:	DictionaryName = cv::aruco::DICT_6X6_50; break;
														case EOpenCVArucoDictionarySize::DictSize100:	DictionaryName = cv::aruco::DICT_6X6_100; break;
														case EOpenCVArucoDictionarySize::DictSize250:	DictionaryName = cv::aruco::DICT_6X6_250; break;
														case EOpenCVArucoDictionarySize::DictSize1000:	DictionaryName = cv::aruco::DICT_6X6_1000; break;
													}
													break;
		case EOpenCVArucoDictionary::Dict7x7:		switch (InDictionarySize)
													{
														case EOpenCVArucoDictionarySize::DictSize50:	DictionaryName = cv::aruco::DICT_7X7_50; break;
														case EOpenCVArucoDictionarySize::DictSize100:	DictionaryName = cv::aruco::DICT_7X7_100; break;
														case EOpenCVArucoDictionarySize::DictSize250:	DictionaryName = cv::aruco::DICT_7X7_250; break;
														case EOpenCVArucoDictionarySize::DictSize1000:	DictionaryName = cv::aruco::DICT_7X7_1000; break;
													}
													break;
		case EOpenCVArucoDictionary::DictOriginal:	DictionaryName = cv::aruco::DICT_ARUCO_ORIGINAL; break;
	}
	cv::Ptr<cv::aruco::Dictionary> Dictionary = cv::aruco::getPredefinedDictionary(DictionaryName);

	// Find all ArUco markers
	std::vector<int> MarkerIds;
	std::vector<std::vector<cv::Point2f>> MarkerCorners;
	const cv::Ptr<cv::aruco::DetectorParameters> DetectorParameters = cv::aruco::DetectorParameters::create();
	cv::aruco::detectMarkers(CvFrameGray, Dictionary, MarkerCorners, MarkerIds, DetectorParameters);

	static const uint32 kMaxCorners = 4;
	const int32 NumMarkersFound = MarkerIds.size();
	if (NumMarkersFound > 0)
	{
		// Validate all return data
		if (NumMarkersFound != MarkerCorners.size())
		{
			UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVArucoDetectMarkers: Number of detected MarkerIds and MarkerCorners don't match");
			return 0;
		}

		for (const std::vector<cv::Point2f>& Corner : MarkerCorners)
		{
			if (Corner.size() != kMaxCorners)
			{
				UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVArucoDetectMarkers: Number of detected corners per marker must be exactly %d", kMaxCorners);
				return 0;
			}
		}

		// Get the estimated 3D pose data relative to the camera if requested
		std::vector<cv::Vec3d> RotationVectors;
		std::vector<cv::Vec3d> TranslationVectors;
		cv::Mat CameraMatrix = cv::Mat::eye(3, 3, CV_64F);
		cv::Mat DistortionCoeffs = cv::Mat::zeros(1, 8, CV_64F);
		if (bEstimatePose)
		{
			CameraMatrix = InLensDistortionParameters.CreateOpenCVCameraMatrix(FVector2D(Size));
			DistortionCoeffs = InLensDistortionParameters.ConvertToOpenCVDistortionCoefficients();

			cv::aruco::estimatePoseSingleMarkers(MarkerCorners, InMarkerLengthInMeters, CameraMatrix, DistortionCoeffs, RotationVectors, TranslationVectors);
		}

		// Draw debug info onto the frame if requested
		if (bDebugDrawMarkers)
		{
			// We need to convert to RGB temporarily because drawDetectedMarkers only supports grayscale or RGB
			cv::cvtColor(CvFrameColor, CvFrameColor, cv::COLOR_RGBA2RGB);
			cv::aruco::drawDetectedMarkers(CvFrameColor, MarkerCorners, MarkerIds, cv::Scalar(0, 255, 255));

			if (bEstimatePose)
			{
				static const float AxisLengthInMeters = 0.05f;
				for (int32 Index = 0; Index < RotationVectors.size(); Index++)
				{
					// drawAxis only does one at a time - it will crash if more than one RotationVector or TranslationVector is passed in
					cv::aruco::drawAxis(CvFrameColor, CameraMatrix, DistortionCoeffs, RotationVectors[Index], TranslationVectors[Index], AxisLengthInMeters);
				}
			}

			cv::cvtColor(CvFrameColor, CvFrameColor, cv::COLOR_RGB2RGBA);

			OutDebugTexture = FOpenCVHelper::TextureFromCvMat(CvFrameColor);
		}

		// Copy the OpenCV data into the Unreal types for return
		OutDetectedMarkers.Reserve(NumMarkersFound);
		for (int32 Index = 0; Index < NumMarkersFound; Index++)
		{
			FOpenCVArucoDetectedMarker NewMarker;
			NewMarker.Id = MarkerIds[Index];
			NewMarker.Corners.Reserve(kMaxCorners);
			for (const cv::Point2f& Corner : MarkerCorners[Index])
			{
				// Normalize each corner to the input image size
				NewMarker.Corners.Add(FVector2D(Corner.x / Size.X, Corner.y / Size.Y));
			}

			// Convert the returned pose into the Unreal coordinate system
			FTransform NewPose = FTransform::Identity;
			if (bEstimatePose)
			{
				// Convert the Rodrigues rotation into a 3x3 rotation matrix
				cv::Mat RotationMatrix = cv::Mat::eye(3, 3, CV_64F);
				cv::Rodrigues(RotationVectors[Index], RotationMatrix);

				// Populate the Unreal rotation matrix
				FMatrix UnrealMatrix = FMatrix::Identity;
				for (int32 Column = 0; Column < 3; Column++)
				{
					UnrealMatrix.SetColumn(Column, FVector(RotationMatrix.at<double>(Column, 0), RotationMatrix.at<double>(Column, 1), RotationMatrix.at<double>(Column, 2)));
				}

				// Scale our translation and populate the FTransform
				static const double MetersToUnrealUnits = 100.0;
				FVector Translation(TranslationVectors[Index][0], TranslationVectors[Index][1], TranslationVectors[Index][2]);
				NewPose = FTransform(UnrealMatrix.GetColumn(0), UnrealMatrix.GetColumn(1), UnrealMatrix.GetColumn(2), Translation * MetersToUnrealUnits);

				// Convert to Unreal axes
				FOpenCVHelper::ConvertOpenCVToUnreal(NewPose);
			}

			NewMarker.Pose = NewPose;
			OutDetectedMarkers.Add(NewMarker);
		}
	}

	return NumMarkersFound;
#else
	{
		UE_LOGF(LogOpenCVBlueprintFunctionLibrary, Error, "OpenCVArucoDetectMarkers: Requires OpenCV");
		return 0;
	}
#endif	// WITH_OPENCV
}
