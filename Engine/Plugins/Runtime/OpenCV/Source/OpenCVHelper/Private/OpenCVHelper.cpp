// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenCVHelper.h"

#include "Engine/Texture2D.h"
#include "Misc/ScopeExit.h"
#include "TextureResource.h"
#include "UObject/Package.h"

#if WITH_OPENCV

#include "PreOpenCVHeaders.h" // IWYU pragma: keep
#include "opencv2/aruco.hpp"
#include "opencv2/calib3d.hpp"
#include "opencv2/aruco/charuco.hpp"
#include "opencv2/imgproc.hpp"
#include "PostOpenCVHeaders.h" // IWYU pragma: keep

#endif	// WITH_OPENCV

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenCVHelper)


DEFINE_LOG_CATEGORY_STATIC(LogOpenCVHelper, Log, All);

#if WITH_OPENCV
namespace
{
	/**
	 * Safely copies data from an OpenCV Mat to a destination buffer.
	 * Handles both contiguous and non-contiguous Mats.
	 * 
	 * @param SourceMat The source OpenCV Mat to copy from
	 * @param DestBuffer The destination buffer to copy to
	 * @param DestBufferSize The size of the destination buffer in bytes
	 * @return true if copy succeeded, false if buffer was too small
	 */
	bool SafeCopyMatToBuffer(const cv::Mat& SourceMat, void* DestBuffer, SIZE_T DestBufferSize)
	{
		const SIZE_T RequiredBytes = SourceMat.total() * SourceMat.elemSize();
		
		if (RequiredBytes > DestBufferSize)
		{
			UE_LOGF(LogOpenCVHelper, Error, "SafeCopyMatToBuffer: Destination buffer too small. Required: %llu bytes, Available: %llu bytes", 
				static_cast<uint64>(RequiredBytes), static_cast<uint64>(DestBufferSize));
			return false;
		}
		
		if (SourceMat.isContinuous())
		{
			// Mat is contiguous, can copy entire buffer at once
			FMemory::Memcpy(DestBuffer, SourceMat.data, RequiredBytes);
		}
		else
		{
			// Mat is not contiguous (e.g., ROI view), copy row by row
			const SIZE_T RowSizeBytes = SIZE_T(SourceMat.cols) * SourceMat.elemSize();
			uint8* DestPtr = static_cast<uint8*>(DestBuffer);
			
			for (int32 Row = 0; Row < SourceMat.rows; ++Row)
			{
				const uint8* SrcPtr = SourceMat.ptr<uint8>(Row);
				FMemory::Memcpy(DestPtr, SrcPtr, RowSizeBytes);
				DestPtr += RowSizeBytes;
			}
		}
		
		return true;
	}
}
#endif // WITH_OPENCV

namespace UE::Aruco::Private
{
#if WITH_OPENCV
	cv::Ptr<cv::aruco::Dictionary> GetArucoDictionary(EArucoDictionary InDictionary)
	{
		switch (InDictionary)
		{
		case EArucoDictionary::DICT_4X4_50:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_4X4_50);		
		case EArucoDictionary::DICT_4X4_100:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_4X4_100);
		case EArucoDictionary::DICT_4X4_250:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_4X4_250);
		case EArucoDictionary::DICT_4X4_1000:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_4X4_1000);
		case EArucoDictionary::DICT_5X5_50:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_5X5_50);
		case EArucoDictionary::DICT_5X5_100:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_5X5_100);
		case EArucoDictionary::DICT_5X5_250:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_5X5_250);
		case EArucoDictionary::DICT_5X5_1000:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_5X5_1000);
		case EArucoDictionary::DICT_6X6_50:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_6X6_50);
		case EArucoDictionary::DICT_6X6_100:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_6X6_100);
		case EArucoDictionary::DICT_6X6_250:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_6X6_250);
		case EArucoDictionary::DICT_6X6_1000:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_6X6_1000);
		case EArucoDictionary::DICT_7X7_50:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_7X7_50);
		case EArucoDictionary::DICT_7X7_100:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_7X7_100);
		case EArucoDictionary::DICT_7X7_250:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_7X7_250);
		case EArucoDictionary::DICT_7X7_1000:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_7X7_1000);
		case EArucoDictionary::DICT_ARUCO_ORIGINAL:
			return cv::aruco::getPredefinedDictionary(cv::aruco::DICT_ARUCO_ORIGINAL);
		default:
			ensureMsgf(false, TEXT("Unhandled EArucoDictionary type. Update this switch statement."));
			break; // Do nothing
		}

		return nullptr;
	}
#endif
}

void FOpenCVHelper::ConvertCoordinateSystem(FTransform& Transform, const EAxis SrcXInDstAxis, const EAxis SrcYInDstAxis, const EAxis SrcZInDstAxis)
{
	// Unreal Engine:
	//   Front : X
	//   Right : Y
	//   Up    : Z
	//
	// OpenCV:
	//   Front : Z
	//   Right : X
	//   Up    : Yn

	FMatrix M12 = FMatrix::Identity;

	M12.SetColumn(0, UnitVectorFromAxisEnum(SrcXInDstAxis));
	M12.SetColumn(1, UnitVectorFromAxisEnum(SrcYInDstAxis));
	M12.SetColumn(2, UnitVectorFromAxisEnum(SrcZInDstAxis));

	Transform.SetFromMatrix(M12.GetTransposed() * Transform.ToMatrixWithScale() * M12);
}

void FOpenCVHelper::ConvertUnrealToOpenCV(FTransform& Transform)
{
	ConvertCoordinateSystem(Transform, EAxis::Y, EAxis::Zn, EAxis::X);
}

void FOpenCVHelper::ConvertOpenCVToUnreal(FTransform& Transform)
{
	ConvertCoordinateSystem(Transform, EAxis::Z, EAxis::X, EAxis::Yn);
}

FVector FOpenCVHelper::ConvertUnrealToOpenCV(const FVector& Vector)
{
	return FVector(Vector.Y, -Vector.Z, Vector.X);
}

FVector FOpenCVHelper::ConvertOpenCVToUnreal(const FVector& Vector)
{
	return FVector(Vector.Z, Vector.X, -Vector.Y);
}

#if WITH_OPENCV
UTexture2D* FOpenCVHelper::TextureFromCvMat(cv::Mat& Mat, const FString* PackagePath, const FName* TextureName)
{
	if ((Mat.cols <= 0) || (Mat.rows <= 0))
	{
		return nullptr;
	}

	// Currently we only support G8 and BGRA8

	if (Mat.depth() != CV_8U)
	{
		return nullptr;
	}

	EPixelFormat PixelFormat;
	ETextureSourceFormat SourceFormat;

	switch (Mat.channels())
	{
	case 1:
		PixelFormat = PF_G8;
		SourceFormat = TSF_G8;
		break;

	case 4:
		PixelFormat = PF_B8G8R8A8;
		SourceFormat = TSF_BGRA8;
		break;

	default:
		return nullptr;
	}

	UTexture2D* Texture = nullptr;

#if WITH_EDITOR
	if (PackagePath && TextureName)
	{
		Texture = NewObject<UTexture2D>(CreatePackage(**PackagePath), *TextureName, RF_Standalone | RF_Public);

		if (!Texture)
		{
			return nullptr;
		}

		Texture->PreEditChange(nullptr);

		const int32 NumSlices = 1;
		const int32 NumMips = 1;

		if (Mat.isContinuous())
		{
			Texture->Source.Init(Mat.cols, Mat.rows, NumSlices, NumMips, SourceFormat, Mat.data);
		}
		else
		{
			// Mat is not contiguous, create a contiguous copy for the UE API
			cv::Mat ContiguousMat;
			Mat.copyTo(ContiguousMat);
			Texture->Source.Init(Mat.cols, Mat.rows, NumSlices, NumMips, SourceFormat, ContiguousMat.data);
		}

		auto IsPowerOfTwo = [](int32 Value)
		{
			return (Value > 0) && ((Value & (Value - 1)) == 0);
		};

		if (!IsPowerOfTwo(Mat.cols) || !IsPowerOfTwo(Mat.rows))
		{
			Texture->MipGenSettings = TMGS_NoMipmaps;
		}

		Texture->SRGB = 0;

		FTextureFormatSettings FormatSettings;

		if (Mat.channels() == 1)
		{
			Texture->CompressionSettings = TextureCompressionSettings::TC_Grayscale;
			Texture->CompressionNoAlpha = true;
		}

		Texture->SetLayerFormatSettings(0, FormatSettings);

		Texture->SetPlatformData(new FTexturePlatformData());
		Texture->GetPlatformData()->SizeX = Mat.cols;
		Texture->GetPlatformData()->SizeY = Mat.rows;
		Texture->GetPlatformData()->PixelFormat = PixelFormat;

		Texture->PostEditChange();
	}
	else
#endif //WITH_EDITOR
	{
		Texture = UTexture2D::CreateTransient(Mat.cols, Mat.rows, PixelFormat);

		if (!Texture)
		{
			return nullptr;
		}

#if WITH_EDITORONLY_DATA
		Texture->MipGenSettings = TMGS_NoMipmaps;
#endif
		Texture->NeverStream = true;
		Texture->SRGB = 0;

		if (Mat.channels() == 1)
		{
			Texture->CompressionSettings = TextureCompressionSettings::TC_Grayscale;
#if WITH_EDITORONLY_DATA
			Texture->CompressionNoAlpha = true;
#endif
		}

		// Copy the pixels from the OpenCV Mat to the Texture
		{
			FTexture2DMipMap& Mip0 = Texture->GetPlatformData()->Mips[0];
			void* TextureData = Mip0.BulkData.Lock(LOCK_READ_WRITE);
			
			ON_SCOPE_EXIT
			{
				if (Mip0.BulkData.IsLocked())
				{
					Mip0.BulkData.Unlock();
				}
			};
			
			if (!TextureData)
			{
				Texture->MarkAsGarbage();
				return nullptr;
			}

			const SIZE_T BufferSize = Mip0.BulkData.GetBulkDataSize();
			if (!SafeCopyMatToBuffer(Mat, TextureData, BufferSize))
			{
				Texture->MarkAsGarbage();
				return nullptr;
			}
		}

		Texture->UpdateResource();
	}

	return Texture;
}

UTexture2D* FOpenCVHelper::TextureFromCvMat(cv::Mat& Mat, UTexture2D* InTexture)
{
	if (!InTexture)
	{
		return TextureFromCvMat(Mat);
	}

	if ((Mat.cols <= 0) || (Mat.rows <= 0))
	{
		return nullptr;
	}

	// Currently we only support G8 and BGRA8

	if (Mat.depth() != CV_8U)
	{
		return nullptr;
	}

	EPixelFormat PixelFormat;

	switch (Mat.channels())
	{
	case 1:
		PixelFormat = PF_G8;
		break;

	case 4:
		PixelFormat = PF_B8G8R8A8;
		break;

	default:
		return nullptr;
	}

	if ((InTexture->GetSizeX() != Mat.cols) || (InTexture->GetSizeY() != Mat.rows) || (InTexture->GetPixelFormat() != PixelFormat))
	{
		return TextureFromCvMat(Mat);
	}

	// Copy the pixels from the OpenCV Mat to the Texture
	{
		FTexture2DMipMap& Mip0 = InTexture->GetPlatformData()->Mips[0];
		void* TextureData = Mip0.BulkData.Lock(LOCK_READ_WRITE);
		
		ON_SCOPE_EXIT
		{
			if (Mip0.BulkData.IsLocked())
			{
				Mip0.BulkData.Unlock();
			}
		};
		
		if (!TextureData)
		{
			return nullptr;
		}

		const SIZE_T BufferSize = Mip0.BulkData.GetBulkDataSize();
		if (!SafeCopyMatToBuffer(Mat, TextureData, BufferSize))
		{
			return nullptr;
		}
	}

	InTexture->UpdateResource();

	return InTexture;
}

void FOpenCVHelper::MakeCameraPoseFromObjectVectors(const cv::Mat& InRotation, const cv::Mat& InTranslation, FTransform& OutTransform)
{
	// The input rotation and translation vectors are both 3x1 vectors that are computed by OpenCV in functions such as calibrateCamera() and solvePnP()
	// Together, they perform a change of basis from object coordinate space to camera coordinate space
	// The desired output transform is the pose of the camera in world space, which can be obtained by inverting/transposing the rotation and translation vectors
	// [R|t]' = [R'|-R'*t]
	// Where R is a 3x3 rotation matrix and t is a 3x1 translation vector

	// Convert input rotation from rodrigues notation to a 3x3 rotation matrix
	cv::Mat RotationMatrix3x3;
	cv::Rodrigues(InRotation, RotationMatrix3x3);

	// Invert/transpose to get the camera orientation
	cv::Mat CameraRotation = RotationMatrix3x3.t();
	cv::Mat CameraTranslation = -RotationMatrix3x3.t() * InTranslation;

	FMatrix TransformationMatrix = FMatrix::Identity;

	// Add the rotation matrix to the transformation matrix
	for (int32 Column = 0; Column < 3; ++Column)
	{
		TransformationMatrix.SetColumn(Column, FVector(CameraRotation.at<double>(Column, 0), CameraRotation.at<double>(Column, 1), CameraRotation.at<double>(Column, 2)));
	}

	// Add the translation vector to the transformation matrix
	TransformationMatrix.M[3][0] = CameraTranslation.at<double>(0);
	TransformationMatrix.M[3][1] = CameraTranslation.at<double>(1);
	TransformationMatrix.M[3][2] = CameraTranslation.at<double>(2);

	OutTransform.SetFromMatrix(TransformationMatrix);

	// Convert the output FTransform to UE's coordinate system
	FOpenCVHelper::ConvertOpenCVToUnreal(OutTransform);
}

void FOpenCVHelper::MakeObjectVectorsFromCameraPose(const FTransform& InTransform, cv::Mat& OutRotation, cv::Mat& OutTranslation)
{
	// Convert the input FTransform to OpenCV's coordinate system
	FTransform CvTransform = InTransform;
	ConvertUnrealToOpenCV(CvTransform);

	const FMatrix TransformationMatrix = CvTransform.ToMatrixNoScale();

	// Extract the translation vector from the transformation matrix
	cv::Mat CameraTranslation = cv::Mat(3, 1, CV_64FC1);
	CameraTranslation.at<double>(0) = TransformationMatrix.M[3][0];
	CameraTranslation.at<double>(1) = TransformationMatrix.M[3][1];
	CameraTranslation.at<double>(2) = TransformationMatrix.M[3][2];

	// Extract the rotation matrix from the transformation matrix
	cv::Mat CameraRotation = cv::Mat(3, 3, CV_64FC1);
	for (int32 Column = 0; Column < 3; ++Column)
	{
		const FVector ColumnVector = TransformationMatrix.GetColumn(Column);
		CameraRotation.at<double>(Column, 0) = ColumnVector.X;
		CameraRotation.at<double>(Column, 1) = ColumnVector.Y;
		CameraRotation.at<double>(Column, 2) = ColumnVector.Z;
	}


	// Invert/transpose to get the rotation and translation that perform a change of basis from object coordinate space to camera coordinate space
	cv::Mat RotationMatrix3x3 = CameraRotation.t();
	OutTranslation = -CameraRotation.inv() * CameraTranslation;

	// Convert the 3x3 rotation matrix to rodrigues notation
	cv::Rodrigues(RotationMatrix3x3, OutRotation);
}
#endif // WITH_OPENCV

bool FOpenCVHelper::IdentifyArucoMarkers(TArray<FColor>& Image, FIntPoint ImageSize, EArucoDictionary DictionaryName, TArray<FArucoMarker>& OutMarkers)
{
#if WITH_OPENCV
	// Validate buffer size matches dimensions
	const int32 RequiredPixels = ImageSize.X * ImageSize.Y;
	if (Image.Num() < RequiredPixels)
	{
		UE_LOGF(LogOpenCVHelper, Error, "IdentifyArucoMarkers: Image buffer too small. Expected %d pixels, got %d", 
			RequiredPixels, Image.Num());
		return false;
	}

	// Initialize an OpenCV matrix header to point at the input image data. 
	// The format for FColor is B8G8R8A8, so the corresponding OpenCV format is CV_8UC4 (8-bit per channel, 4 channels)
	cv::Mat ImageMat = cv::Mat(ImageSize.Y, ImageSize.X, CV_8UC4, Image.GetData());

	// Convert the image to grayscale before attempting to detect markers
	cv::Mat GrayImage;
	cv::cvtColor(ImageMat, GrayImage, cv::COLOR_RGBA2GRAY);

	// We do not know the number of markers expected in the image, so we cannot yet reserve the necessary space for our marker data and create a convenient matrix header for that data. 
	// Instead, we will pass unitialized vectors to opencv and copy the data into our final data structure after the markers have been identified. 
 	std::vector<int> MarkerIds;
 	std::vector<std::vector<cv::Point2f>> Corners;

	cv::Ptr<cv::aruco::Dictionary> Dictionary = UE::Aruco::Private::GetArucoDictionary(DictionaryName);
	if (!Dictionary)
	{
		return false;
	}

	cv::Ptr<cv::aruco::DetectorParameters> DetectorParameters = cv::aruco::DetectorParameters::create();
	DetectorParameters->cornerRefinementMethod = cv::aruco::CORNER_REFINE_SUBPIX;

	cv::aruco::detectMarkers(GrayImage, Dictionary, Corners, MarkerIds, DetectorParameters);

	OutMarkers.Empty();

	const int32 NumMarkers = MarkerIds.size();

	if (NumMarkers < 1)
	{
		return false;
	}

	OutMarkers.Reserve(NumMarkers);

	// Copy the detected marker data into the final data structure
	for (int32 MarkerIndex = 0; MarkerIndex < NumMarkers; ++MarkerIndex)
	{
		FArucoMarker NewMarker;
		NewMarker.MarkerID = MarkerIds[MarkerIndex];

		const std::vector<cv::Point2f>& MarkerCorners = Corners[MarkerIndex];

		NewMarker.Corners[0] = FVector2f(MarkerCorners[0].x, MarkerCorners[0].y); // TopLeft
		NewMarker.Corners[1] = FVector2f(MarkerCorners[1].x, MarkerCorners[1].y); // TopRight
		NewMarker.Corners[2] = FVector2f(MarkerCorners[2].x, MarkerCorners[2].y); // BottomRight
		NewMarker.Corners[3] = FVector2f(MarkerCorners[3].x, MarkerCorners[3].y); // BottomLeft

		OutMarkers.Add(NewMarker);
	}

	return true;
#else
	return false;
#endif // WITH_OPENCV
}

bool FOpenCVHelper::DrawArucoMarkers(const TArray<FArucoMarker>& Markers, UTexture2D* DebugTexture)
{
#if WITH_OPENCV
	if (!DebugTexture || Markers.IsEmpty())
	{
		return false;
	}

	const int32 NumMarkers = Markers.Num();

	std::vector<int> MarkerIds;
	MarkerIds.reserve(NumMarkers);

	std::vector<std::vector<cv::Point2f>> Corners;
	Corners.reserve(NumMarkers);

	// Copy the detected marker data into the final data structure
	for (int32 MarkerIndex = 0; MarkerIndex < NumMarkers; ++MarkerIndex)
	{
		const FArucoMarker& Marker = Markers[MarkerIndex];

		MarkerIds.push_back(Marker.MarkerID);

		std::vector<cv::Point2f> MarkerCorners;
		MarkerCorners.reserve(4);

		MarkerCorners.push_back(cv::Point2f(Marker.Corners[0].X, Marker.Corners[0].Y));
		MarkerCorners.push_back(cv::Point2f(Marker.Corners[1].X, Marker.Corners[1].Y));
		MarkerCorners.push_back(cv::Point2f(Marker.Corners[2].X, Marker.Corners[2].Y));
		MarkerCorners.push_back(cv::Point2f(Marker.Corners[3].X, Marker.Corners[3].Y));

		Corners.push_back(MarkerCorners);
	}

	{
		FTexture2DMipMap& Mip0 = DebugTexture->GetPlatformData()->Mips[0];
		void* TextureData = Mip0.BulkData.Lock(LOCK_READ_WRITE);
		
		ON_SCOPE_EXIT
		{
			if (Mip0.BulkData.IsLocked())
			{
				Mip0.BulkData.Unlock();
			}
		};
		
		if (!TextureData)
		{
			return false;
		}

		// Create a matrix header pointing to the raw texture data, copy the debug marker outlines into it, and update the texture resource
		cv::Mat TextureMat = cv::Mat(DebugTexture->GetSizeY(), DebugTexture->GetSizeX(), CV_8UC4, TextureData);

		// We need to convert from RGBA to RGB temporarily because cv::aruco::drawDetectedMarkers only supports images with 1 or 3 color channels (not 4)
		// We use a second matrix header because cvtColor will make a copy of the original data each time it changes the color format and we do not want to lose the reference to the original TextureData pointer.
		cv::Mat DebugMat;
		TextureMat.copyTo(DebugMat);

		cv::cvtColor(DebugMat, DebugMat, cv::COLOR_RGBA2RGB);
		cv::aruco::drawDetectedMarkers(DebugMat, Corners, MarkerIds);
		cv::cvtColor(DebugMat, DebugMat, cv::COLOR_RGB2RGBA);

		DebugMat.copyTo(TextureMat);
	}

	DebugTexture->UpdateResource();

	return true;
#else
	return false;
#endif // WITH_OPENCV
}

bool FOpenCVHelper::IdentifyCheckerboard(TArray<FColor>& Image, FIntPoint ImageSize, FIntPoint CheckerboardDimensions, TArray<FVector2f>& OutCorners)
{
	FIntRect FullImageRect = FIntRect(FIntPoint(0), ImageSize);
	return IdentifyCheckerboard(Image, ImageSize, FullImageRect, CheckerboardDimensions, OutCorners);
}

bool FOpenCVHelper::IdentifyCheckerboard(TArray<FColor>& Image, FIntPoint ImageSize, FIntRect RegionOfInterest, FIntPoint CheckerboardDimensions, TArray<FVector2f>& OutCorners)
{
#if WITH_OPENCV
	// Validate buffer size matches dimensions
	const int32 RequiredPixels = ImageSize.X * ImageSize.Y;
	if (Image.Num() < RequiredPixels)
	{
		UE_LOGF(LogOpenCVHelper, Error, "IdentifyCheckerboard: Image buffer too small. Expected %d pixels, got %d", 
			RequiredPixels, Image.Num());
		return false;
	}

	// Initialize an OpenCV matrix header to point at the input image data. 
	// The format for FColor is B8G8R8A8, so the corresponding OpenCV format is CV_8UC4 (8-bit per channel, 4 channels)
	const cv::Mat WholeImageMat = cv::Mat(ImageSize.Y, ImageSize.X, CV_8UC4, Image.GetData());

	// Sanitize the ROI to ensure that it lies completely within the bounds of the full size image
	RegionOfInterest.Min.X = FMath::Clamp(RegionOfInterest.Min.X, 0, ImageSize.X);
	RegionOfInterest.Min.Y = FMath::Clamp(RegionOfInterest.Min.Y, 0, ImageSize.Y);
	RegionOfInterest.Max.X = FMath::Clamp(RegionOfInterest.Max.X, RegionOfInterest.Min.X + 1, ImageSize.X);
	RegionOfInterest.Max.Y = FMath::Clamp(RegionOfInterest.Max.Y, RegionOfInterest.Min.Y + 1, ImageSize.Y);

	if ((RegionOfInterest.Width() <= 0) || (RegionOfInterest.Height() <= 0))
	{
		OutCorners.Empty();
		return false;
	}

	// Create a header for the region of interest
	const cv::Rect CvROI = cv::Rect(RegionOfInterest.Min.X, RegionOfInterest.Min.Y, RegionOfInterest.Size().X, RegionOfInterest.Size().Y);
	const cv::Mat ImageMat = cv::Mat(WholeImageMat, CvROI);

	// Convert the image to grayscale before attempting to detect checkerboard corners
	cv::Mat GrayImage;
	cv::cvtColor(ImageMat, GrayImage, cv::COLOR_RGBA2GRAY);

	const cv::Size CheckerboardSize = cv::Size(CheckerboardDimensions.X, CheckerboardDimensions.Y);
	const int32 NumExpectedCorners = CheckerboardDimensions.X * CheckerboardDimensions.Y;

	// ADAPTIVE_THRESH and NORMALIZE_IMAGE are both default options to aid in corner detection. 
	// FAST_CHECK will quickly determine if there is any checkerboard in the image and early-out quickly if not
	const int32 FindCornersFlags = cv::CALIB_CB_ADAPTIVE_THRESH | cv::CALIB_CB_NORMALIZE_IMAGE | cv::CALIB_CB_FAST_CHECK;

	// Initialize the output array to hold the number of expected corners, then create a matrix header to the data to pass to OpenCV
	OutCorners.Init(FVector2f(), NumExpectedCorners);
	cv::Mat CornerMat = cv::Mat(NumExpectedCorners, 1, CV_32FC2, (void*)OutCorners.GetData());

	const bool bFoundCorners = cv::findChessboardCorners(GrayImage, CheckerboardSize, CornerMat, FindCornersFlags);

	if (!bFoundCorners || OutCorners.Num() != NumExpectedCorners)
	{
		OutCorners.Empty();
		return false;
	}

	// Attempt to refine the corner detection to subpixel accuracy. Algorithm settings are all defaults from the OpenCV documentation.
	const cv::TermCriteria Criteria = cv::TermCriteria(cv::TermCriteria::Type::EPS | cv::TermCriteria::Type::COUNT, 30, 0.001);
	const cv::Size HalfSearchWindowSize = cv::Size(11, 11);
	const cv::Size HalfDeadZoneSize = cv::Size(-1, -1);
	cv::cornerSubPix(GrayImage, CornerMat, HalfSearchWindowSize, HalfDeadZoneSize, Criteria);

	// The detected corner array can begin with either the TopLeft or BottomRight corner. If the BottomRight corner is first, reverse the order.
	if ((NumExpectedCorners >= 2) && (OutCorners[0].Y > OutCorners[NumExpectedCorners - 1].Y))
	{
		Algo::Reverse(OutCorners);
	}

	// Adjust the detected corners to be relative to the full image, not the ROI
	for (FVector2f& Corner : OutCorners)
	{
		Corner += RegionOfInterest.Min;
	}

	return true;
#else
	return false;
#endif // WITH_OPENCV
}

bool FOpenCVHelper::DrawCheckerboardCorners(const TArray<FVector2f>& Corners, FIntPoint CheckerboardDimensions, UTexture2D* DebugTexture)
{
#if WITH_OPENCV
	if (!DebugTexture || Corners.IsEmpty())
	{
		return false;
	}

	cv::Mat CornerMat(Corners.Num(), 1, CV_32FC2, (void*)Corners.GetData());

	{
		FTexture2DMipMap& Mip0 = DebugTexture->GetPlatformData()->Mips[0];
		void* TextureData = Mip0.BulkData.Lock(LOCK_READ_WRITE);
		
		ON_SCOPE_EXIT
		{
			if (Mip0.BulkData.IsLocked())
			{
				Mip0.BulkData.Unlock();
			}
		};
		
		if (!TextureData)
		{
			return false;
		}

		// Create a matrix header pointing to the raw texture data, draw the debug pattern, and update the texture resource
		const FIntPoint TextureSize = FIntPoint(DebugTexture->GetSizeX(), DebugTexture->GetSizeY());
		cv::Mat TextureMat = cv::Mat(TextureSize.Y, TextureSize.X, CV_8UC4, TextureData);

		constexpr bool bFoundPattern = true;
		cv::drawChessboardCorners(TextureMat, cv::Size(CheckerboardDimensions.X, CheckerboardDimensions.Y), CornerMat, bFoundPattern);
	}

	DebugTexture->UpdateResource();

	return true;
#else
	return false;
#endif // WITH_OPENCV
}

bool FOpenCVHelper::DrawCheckerboardCorners(const TArray<FVector2D>& Corners, FIntPoint CheckerboardDimensions, UTexture2D* DebugTexture)
{
	TArray<FVector2f> CornersFloat;
	CornersFloat.Reserve(Corners.Num());
	for (const FVector2D& Corner : Corners)
	{
		CornersFloat.Add(FVector2f(Corner.X, Corner.Y));
	}

	return DrawCheckerboardCorners(CornersFloat, CheckerboardDimensions, DebugTexture);
}

bool FOpenCVHelper::SolvePnP(const TArray<FVector>& ObjectPoints, const TArray<FVector2f>& ImagePoints, const FVector2D& FocalLength, const FVector2D& ImageCenter, const TArray<float>& DistortionParameters, FTransform& OutCameraPose)
{
#if WITH_OPENCV
	const int32 NumPoints = ObjectPoints.Num();
	if (!ensureMsgf((ImagePoints.Num() == NumPoints), TEXT("The number of 3D object points must match the number of 2D image points being passed to solvePnP()")))
	{
		return false;
	}
	
	// For non-planar sets of 3D points, solvePnP requires a minimum of 6 points to compute the direct linear transformation (DLT)
	constexpr int32 MinimumPoints = 6;
	if (NumPoints < MinimumPoints)
	{
		UE_LOGF(LogOpenCVHelper, Error, "SolvePnP requires a minimum of 6 3D/2D point correspondences, but only %d were provided", NumPoints);
		return false;
	}

	// cv::solvePnP() will only accept spherical distortion parameters, but it accepts a variable number of parameters based on how much of the distortion model is used
	// We need to guard against an incorrect number of parameters to avoid crashing in the opencv module
	const int32 NumDistortionParameters = DistortionParameters.Num();
	const bool bCorrectNumDistortionParameters = (NumDistortionParameters == 0) || (NumDistortionParameters == 4)  || (NumDistortionParameters == 5) || 
		                                         (NumDistortionParameters == 8) || (NumDistortionParameters == 12) || (NumDistortionParameters == 14);
	
	if (!ensureMsgf(bCorrectNumDistortionParameters, TEXT("The number of distortion parameters being passed to solvePnP() is invalid")))
	{
		return false;
	}

	// Convert from UE coordinates to OpenCV coordinates
	TArray<FVector> CvObjectPoints;
	CvObjectPoints.Reserve(NumPoints);

	for (const FVector& ObjectPoint : ObjectPoints)
	{
		CvObjectPoints.Add(FOpenCVHelper::ConvertUnrealToOpenCV(ObjectPoint));
	}

	cv::Mat ObjectPointsMat(NumPoints, 1, CV_64FC3, (void*)CvObjectPoints.GetData());
	cv::Mat ImagePointsMat(NumPoints, 1, CV_32FC2, (void*)ImagePoints.GetData());
	cv::Mat DistortionMat(NumDistortionParameters, 1, CV_32FC1, (void*)DistortionParameters.GetData());

	// Initialize the camera matrix that will be used in each call to projectPoints()
	cv::Mat CameraMatrix = cv::Mat::eye(3, 3, CV_64F);

	CameraMatrix.at<double>(0, 0) = FocalLength.X;
	CameraMatrix.at<double>(1, 1) = FocalLength.Y;
	CameraMatrix.at<double>(0, 2) = ImageCenter.X;
	CameraMatrix.at<double>(1, 2) = ImageCenter.Y;

	// Solve for camera position
	cv::Mat Rotation;
	cv::Mat Translation;

	// We send no distortion parameters, because Points2d was manually undistorted already
	if (!cv::solvePnP(ObjectPointsMat, ImagePointsMat, CameraMatrix, DistortionMat, Rotation, Translation))
	{
		return false;
	}

	// Convert the OpenCV rotation and translation vectors into an FTransform in UE's coordinate system
	MakeCameraPoseFromObjectVectors(Rotation, Translation, OutCameraPose);

	return true;
#else
	return false;
#endif // WITH_OPENCV
}

bool FOpenCVHelper::ProjectPoints(const TArray<FVector>& ObjectPoints, const FVector2D& FocalLength, const FVector2D& ImageCenter, const TArray<float>& DistortionParameters, const FTransform& CameraPose, TArray<FVector2f>& OutImagePoints)
{
	TArray<FVector2D> OutImagePointsDoublePrecision;
	if (ProjectPoints(ObjectPoints, FocalLength, ImageCenter, DistortionParameters, CameraPose, OutImagePointsDoublePrecision))
	{
		for (const FVector2D& Point : OutImagePointsDoublePrecision)
		{
			OutImagePoints.Add(FVector2f(Point.X, Point.Y));
		}

		return true;
	}
	return false;
}

bool FOpenCVHelper::ProjectPoints(const TArray<FVector>& ObjectPoints, const FVector2D& FocalLength, const FVector2D& ImageCenter, const TArray<float>& DistortionParameters, const FTransform& CameraPose, TArray<FVector2D>& OutImagePoints)
{
#if WITH_OPENCV
	const int32 NumPoints = ObjectPoints.Num();

	// Convert from UE coordinates to OpenCV coordinates
	TArray<FVector> CvObjectPoints;
	CvObjectPoints.Reserve(NumPoints);

	for (const FVector& ObjectPoint : ObjectPoints)
	{
		CvObjectPoints.Add(FOpenCVHelper::ConvertUnrealToOpenCV(ObjectPoint));
	}

	cv::Mat ObjectPointsMat = cv::Mat(NumPoints, 1, CV_64FC3, (void*)CvObjectPoints.GetData());

	cv::Mat Rotation;
	cv::Mat Translation;
	FOpenCVHelper::MakeObjectVectorsFromCameraPose(CameraPose, Rotation, Translation);

	// Initialize the camera matrix that will be used in each call to projectPoints()
	cv::Mat CameraMatrix = cv::Mat::eye(3, 3, CV_64F);

	CameraMatrix.at<double>(0, 0) = FocalLength.X;
	CameraMatrix.at<double>(1, 1) = FocalLength.Y;
	CameraMatrix.at<double>(0, 2) = ImageCenter.X;
	CameraMatrix.at<double>(1, 2) = ImageCenter.Y;

	cv::Mat DistortionParametersMat = cv::Mat(DistortionParameters.Num(), 1, CV_32FC1, (void*)DistortionParameters.GetData());

	OutImagePoints.Init(FVector2D(0.0, 0.0), NumPoints);
	cv::Mat ImagePointsMat = cv::Mat(NumPoints, 1, CV_64FC2, (void*)OutImagePoints.GetData());

	cv::projectPoints(ObjectPointsMat, Rotation, Translation, CameraMatrix, DistortionParametersMat, ImagePointsMat);

	return true;
#else
	return false;
#endif // WITH_OPENCV
}

bool FOpenCVHelper::FitLine3D(const TArray<FVector>& InPoints, FVector& OutLine, FVector& OutPointOnLine)
{
#if WITH_OPENCV
	const int32 NumPoints = InPoints.Num();

	if (NumPoints < 2)
	{
		return false;
	}

	cv::Mat PointsMat(NumPoints, 1, CV_64FC3, (void*)InPoints.GetData());

	// Find a best fit line between the 3D points, producing a line (the optical axis) and a point on that line
	constexpr int CParam = 0; // fitline parameter of 0 will try to find an optimal value for the simple euclidean distance method of DIST_L2
	constexpr double RadiusAccuracy = 0.01; // cv documentation specifies 0.01 as a good default value
	constexpr double AngleAccuracy = 0.01; // cv documentation specifies 0.01 as a good default value

	cv::Vec6f LineAndPoint;
	cv::fitLine(PointsMat, LineAndPoint, cv::DIST_L2, CParam, RadiusAccuracy, AngleAccuracy);

	OutLine = FVector(LineAndPoint[0], LineAndPoint[1], LineAndPoint[2]);
	OutPointOnLine = FVector(LineAndPoint[3], LineAndPoint[4], LineAndPoint[5]);

	return true;
#else
	return false;
#endif // WITH_OPENCV
}

#if WITH_OPENCV
double FOpenCVHelper::ComputeReprojectionError(const FTransform& CameraPose, const cv::Mat& CameraIntrinsicMatrix, const std::vector<cv::Point3f>& Points3d, const std::vector<cv::Point2f>& Points2d)
{
	// Ensure that the number of point correspondences is valid
	const int32 NumPoints3d = Points3d.size();
	const int32 NumPoints2d = Points2d.size();
	if ((NumPoints3d == 0) || (NumPoints2d == 0) || (NumPoints3d != NumPoints2d))
	{
		return -1.0;
	}

	const FMatrix CameraPoseMatrix = CameraPose.ToMatrixNoScale();

	const cv::Mat Tcam = (cv::Mat_<double>(3, 1) << CameraPoseMatrix.M[3][0], CameraPoseMatrix.M[3][1], CameraPoseMatrix.M[3][2]);

	cv::Mat Rcam = cv::Mat::zeros(3, 3, cv::DataType<double>::type);
	for (int32 Column = 0; Column < 3; ++Column)
	{
		FVector ColVec = CameraPoseMatrix.GetColumn(Column);
		Rcam.at<double>(Column, 0) = ColVec.X;
		Rcam.at<double>(Column, 1) = ColVec.Y;
		Rcam.at<double>(Column, 2) = ColVec.Z;
	}

	const cv::Mat Robj = Rcam.t();

	cv::Mat Rrod;
	cv::Rodrigues(Robj, Rrod);

	const cv::Mat Tobj = -Rcam.inv() * Tcam;

	std::vector<cv::Point2f> ReprojectedPoints2d;

	// The 2D points will be compared against the undistorted 2D points, so the distortion coefficients can be ignored
	cv::projectPoints(Points3d, Rrod, Tobj, CameraIntrinsicMatrix, cv::noArray(), ReprojectedPoints2d);

	if (ReprojectedPoints2d.size() != NumPoints2d)
	{
		return -1.0;
	}

	// Compute euclidean distance between captured 2D points and reprojected 2D points to measure reprojection error
	double ReprojectionError = 0.0;
	for (int32 Index = 0; Index < NumPoints2d; ++Index)
	{
		const cv::Point2f& A = Points2d[Index];
		const cv::Point2f& B = ReprojectedPoints2d[Index];
		const cv::Point2f Diff = A - B;

		ReprojectionError += (Diff.x * Diff.x) + (Diff.y * Diff.y); // cv::norm with NORM_L2SQR
	}

	return ReprojectionError;
}
#endif

double FOpenCVHelper::ComputeReprojectionError(const TArray<FVector>& ObjectPoints, const TArray<FVector2f>& ImagePoints, const FVector2D& FocalLength, const FVector2D& ImageCenter, const FTransform& CameraPose)
{
#if WITH_OPENCV
	// Ensure that the number of point correspondences is valid
	const int32 NumPoints3d = ObjectPoints.Num();
	const int32 NumPoints2d = ImagePoints.Num();
	if ((NumPoints3d == 0) || (NumPoints2d == 0) || (NumPoints3d != NumPoints2d))
	{
		return -1.0;
	}

	// Initialize the camera matrix that will be used in each call to projectPoints()
	cv::Mat CameraMatrix = cv::Mat::eye(3, 3, CV_64F);

	CameraMatrix.at<double>(0, 0) = FocalLength.X;
	CameraMatrix.at<double>(1, 1) = FocalLength.Y;
	CameraMatrix.at<double>(0, 2) = ImageCenter.X;
	CameraMatrix.at<double>(1, 2) = ImageCenter.Y;

	cv::Mat Rotation;
	cv::Mat Translation;
	FOpenCVHelper::MakeObjectVectorsFromCameraPose(CameraPose, Rotation, Translation);

	// cv::projectPoints requires that the 3D points and 2D points have the same bit depth, so we have to make a copy of the ObjectPoints with only 32-bits of precision
	TArray<FVector3f> CvObjectPoints;
	for (const FVector& Point : ObjectPoints)
	{
		FVector CvPoint = ConvertUnrealToOpenCV(Point);
		CvObjectPoints.Add(FVector3f(CvPoint.X, CvPoint.Y, CvPoint.Z));
	}

	TArray<FVector2f> ProjectedPoints;
	ProjectedPoints.Init(FVector2f(), NumPoints2d);

	cv::Mat ObjectPointsMat = cv::Mat(ObjectPoints.Num(), 1, CV_32FC3, (void*)CvObjectPoints.GetData());
	cv::Mat ProjectedPointsMat = cv::Mat(ProjectedPoints.Num(), 1, CV_32FC2, (void*)ProjectedPoints.GetData());

	// The 2D points will be compared against the undistorted 2D points, so the distortion coefficients can be ignored
	cv::projectPoints(ObjectPointsMat, Rotation, Translation, CameraMatrix, cv::noArray(), ProjectedPointsMat);

	if (ProjectedPoints.Num() != NumPoints2d)
	{
		return -1.0;
	}

	// Compute euclidean distance between captured 2D points and reprojected 2D points to measure reprojection error
	double ReprojectionError = 0.0;
	for (int32 Index = 0; Index < NumPoints2d; ++Index)
	{
		const FVector2f Diff = ProjectedPoints[Index] - ImagePoints[Index];
		ReprojectionError += (Diff.X * Diff.X) + (Diff.Y * Diff.Y); // cv::norm with NORM_L2SQR
	}

	return ReprojectionError;
#else 
	return -1.0f;
#endif // WITH_OPENCV
}

#if WITH_OPENCV
cv::Mat FOpenCVLensDistortionParametersBase::ConvertToOpenCVDistortionCoefficients() const
{
	if (bUseFisheyeModel)
	{
		cv::Mat DistortionCoefficients(1, 4, CV_64F);
		DistortionCoefficients.at<double>(0) = K1;
		DistortionCoefficients.at<double>(1) = K2;
		DistortionCoefficients.at<double>(2) = K3;
		DistortionCoefficients.at<double>(3) = K4;
		return DistortionCoefficients;
	}
	else
	{
		cv::Mat DistortionCoefficients(1, 8, CV_64F);
		DistortionCoefficients.at<double>(0) = K1;
		DistortionCoefficients.at<double>(1) = K2;
		DistortionCoefficients.at<double>(2) = P1;
		DistortionCoefficients.at<double>(3) = P2;
		DistortionCoefficients.at<double>(4) = K3;
		DistortionCoefficients.at<double>(5) = K4;
		DistortionCoefficients.at<double>(6) = K5;
		DistortionCoefficients.at<double>(7) = K6;
		return DistortionCoefficients;
	}
}

cv::Mat FOpenCVLensDistortionParametersBase::CreateOpenCVCameraMatrix(const FVector2D& InImageSize) const
{
	cv::Mat CameraMatrix = cv::Mat::eye(3, 3, CV_64F);
	CameraMatrix.at<double>(0, 0) = F.X * InImageSize.X;
	CameraMatrix.at<double>(1, 1) = F.Y * InImageSize.Y;
	CameraMatrix.at<double>(0, 2) = C.X * InImageSize.X;
	CameraMatrix.at<double>(1, 2) = C.Y * InImageSize.Y;
	return CameraMatrix;
}
#endif // WITH_OPENCV


bool FOpenCVHelper::IdentifyCharucoCorners(TArray<FColor>& Image, FIntPoint ImageSize, const FCharucoBoardConfig& BoardConfig, FCharucoCorners& OutCorners, const FIntRect& ROI)
{
#if WITH_OPENCV
	if (Image.IsEmpty())
	{
		UE_LOGF(LogOpenCVHelper, Error, "IdentifyCharucoCorners: Image is empty");
		return false;
	}

	// Validate buffer size matches dimensions
	const int32 RequiredPixels = ImageSize.X * ImageSize.Y;
	if (Image.Num() < RequiredPixels)
	{
		UE_LOGF(LogOpenCVHelper, Error, "IdentifyCharucoCorners: Image buffer too small. Expected %d pixels, got %d", 
			RequiredPixels, Image.Num());
		return false;
	}

	// Validate board configuration parameters
	if (BoardConfig.SquaresX <= 0 || BoardConfig.SquaresY <= 0)
	{
		UE_LOGF(LogOpenCVHelper, Error, "IdentifyCharucoCorners: Invalid board dimensions (%d x %d squares). Must be positive.", 
			BoardConfig.SquaresX, BoardConfig.SquaresY);
		return false;
	}

	if (BoardConfig.SquareSize <= 0.0f || BoardConfig.MarkerSize <= 0.0f)
	{
		UE_LOGF(LogOpenCVHelper, Error, "IdentifyCharucoCorners: Invalid sizes (SquareSize: %f, MarkerSize: %f). Must be positive.", 
			BoardConfig.SquareSize, BoardConfig.MarkerSize);
		return false;
	}

	if (BoardConfig.MarkerSize >= BoardConfig.SquareSize)
	{
		UE_LOGF(LogOpenCVHelper, Error, "IdentifyCharucoCorners: MarkerSize (%f) must be smaller than SquareSize (%f).", 
			BoardConfig.MarkerSize, BoardConfig.SquareSize);
		return false;
	}

	// Get ArUco dictionary

	cv::Ptr<cv::aruco::Dictionary> Dictionary = UE::Aruco::Private::GetArucoDictionary(BoardConfig.Dictionary);

	if (Dictionary == nullptr)
	{
		UE_LOGF(LogOpenCVHelper, Error, "IdentifyCharucoCorners: Invalid ArUco dictionary");
		return false;
	}

	// Create Charuco board

	cv::Ptr<cv::aruco::CharucoBoard> CharucoBoard = cv::aruco::CharucoBoard::create(
		BoardConfig.SquaresX, 
		BoardConfig.SquaresY, 
		BoardConfig.SquareSize, 
		BoardConfig.MarkerSize, 
		Dictionary
	);

	// Convert image to OpenCV format

	cv::Mat CvImage(ImageSize.Y, ImageSize.X, CV_8UC4, Image.GetData());
	cv::Mat GrayImage;
	cv::cvtColor(CvImage, GrayImage, cv::COLOR_BGRA2GRAY);

	// Apply ROI (if any)

	bool bUsingROI = (ROI.Width() > 0 && ROI.Height() > 0);
	cv::Mat ProcessingImage = GrayImage;
	FIntPoint ROIOffset = FIntPoint::ZeroValue;
	
	if (bUsingROI)
	{
		// Clamp ROI to image bounds

		int ROILeft = FMath::Clamp(ROI.Min.X, 0, ImageSize.X - 1);
		int ROITop = FMath::Clamp(ROI.Min.Y, 0, ImageSize.Y - 1);
		int ROIRight = FMath::Clamp(ROI.Max.X, ROILeft + 1, ImageSize.X);
		int ROIBottom = FMath::Clamp(ROI.Max.Y, ROITop + 1, ImageSize.Y);
		
		ROIOffset = FIntPoint(ROILeft, ROITop);
		cv::Rect ROIRect(ROILeft, ROITop, ROIRight - ROILeft, ROIBottom - ROITop);

		ProcessingImage = GrayImage(ROIRect);
		
		UE_LOGF(LogOpenCVHelper, Verbose, "IdentifyCharucoCorners: Using ROI (%d,%d) to (%d,%d), size %dx%d", 
			ROILeft, ROITop, ROIRight, ROIBottom, ProcessingImage.cols, ProcessingImage.rows);
	}

	// First detect ArUco markers

	std::vector<int> MarkerIds;
	std::vector<std::vector<cv::Point2f>> MarkerCorners;
	cv::Ptr<cv::aruco::DetectorParameters> Parameters = cv::aruco::DetectorParameters::create();

	cv::aruco::detectMarkers(ProcessingImage, Dictionary, MarkerCorners, MarkerIds, Parameters);

	// If no markers found, return false

	if (MarkerIds.empty())
	{
		UE_LOGF(LogOpenCVHelper, Warning, "IdentifyCharucoCorners: No ArUco markers detected");
		return false;
	}

	// Store detected ArUco markers (transform coordinates back to full image space if using ROI)

	OutCorners.Markers.Empty();

	for (size_t MarkerIdx = 0; MarkerIdx < MarkerIds.size(); MarkerIdx++)
	{
		FArucoMarker Marker;
		Marker.MarkerID = MarkerIds[MarkerIdx];

		constexpr int32 CornersPerMarker = 4;

		for (int32 CornerIdx = 0; CornerIdx < CornersPerMarker; CornerIdx++)
		{
			const float X = MarkerCorners[MarkerIdx][CornerIdx].x + (bUsingROI ? ROIOffset.X : 0);
			const float Y = MarkerCorners[MarkerIdx][CornerIdx].y + (bUsingROI ? ROIOffset.Y : 0);

			Marker.Corners[CornerIdx] = FVector2f(X, Y);
		}

		OutCorners.Markers.Add(Marker);
	}

	// Interpolate Charuco corners
	std::vector<cv::Point2f> CharucoCorners;
	std::vector<int> CharucoIds;
	
	if (MarkerIds.size() > 0)
	{
		cv::aruco::interpolateCornersCharuco(MarkerCorners, MarkerIds, ProcessingImage, CharucoBoard, CharucoCorners, CharucoIds);
	}

	// If no Charuco corners found, return false
	if (CharucoIds.empty())
	{
		UE_LOGF(LogOpenCVHelper, Warning, "IdentifyCharucoCorners: No Charuco corners detected");
		return false;
	}

	// Convert to output format (transform coordinates back to full image space if using ROI)
	OutCorners.Corners.Empty();
	OutCorners.CornerIds.Empty();
	
	for (size_t CornerIdx = 0; CornerIdx < CharucoCorners.size(); CornerIdx++)
	{
		const float X = CharucoCorners[CornerIdx].x + (bUsingROI ? ROIOffset.X : 0);
		const float Y = CharucoCorners[CornerIdx].y + (bUsingROI ? ROIOffset.Y : 0);

		OutCorners.Corners.Add(FVector2f(X, Y));
		OutCorners.CornerIds.Add(CharucoIds[CornerIdx]);
	}

	UE_LOGF(LogOpenCVHelper, Verbose, "IdentifyCharucoCorners: Detected %d Charuco corners and %d ArUco markers", 
		OutCorners.Corners.Num(), OutCorners.Markers.Num());

	return true;
#else
	UE_LOGF(LogOpenCVHelper, Error, "IdentifyCharucoCorners: OpenCV not available");
	return false;
#endif
}


bool FOpenCVHelper::DrawCharucoCorners(const FCharucoCorners& Corners, UTexture2D* DebugTexture)
{
#if WITH_OPENCV

	if (!DebugTexture)
	{
		UE_LOGF(LogOpenCVHelper, Error, "DrawCharucoCorners: Invalid texture (null)");
		return false;
	}

	FTexturePlatformData* PlatData = DebugTexture->GetPlatformData();
	if (!PlatData || PlatData->Mips.Num() == 0)
	{
		UE_LOGF(LogOpenCVHelper, Error, "DrawCharucoCorners: Missing platform data or mips");
		return false;
	}

	if (PlatData->PixelFormat != PF_B8G8R8A8)
	{
		UE_LOGF(LogOpenCVHelper, Error, "DrawCharucoCorners: Unsupported PixelFormat. Expected PF_B8G8R8A8, got %d", (int32)PlatData->PixelFormat);
		return false;
	}

	const int32 Width = DebugTexture->GetSizeX();
	const int32 Height = DebugTexture->GetSizeY();

	if (Width <= 0 || Height <= 0)
	{
		UE_LOGF(LogOpenCVHelper, Error, "DrawCharucoCorners: Invalid texture size %dx%d", Width, Height);
		return false;
	}

	FTexture2DMipMap& Mip = PlatData->Mips[0];
	uint8* TextureData = static_cast<uint8*>(Mip.BulkData.Lock(LOCK_READ_WRITE));

	ON_SCOPE_EXIT
	{
		if (Mip.BulkData.IsLocked())
		{
			Mip.BulkData.Unlock();
		}
	};

	if (!TextureData)
	{
		UE_LOGF(LogOpenCVHelper, Error, "DrawCharucoCorners: Failed to lock texture bulk data");
		return false;
	}

	const int32 BytesPerPixel = 4;

	const SIZE_T ExpectedSize = SIZE_T(Width) * SIZE_T(Height) * SIZE_T(BytesPerPixel);
	const SIZE_T BulkSize = Mip.BulkData.GetBulkDataSize();

	if (BulkSize < ExpectedSize)
	{
		UE_LOGF(LogOpenCVHelper, Error, "DrawCharucoCorners: BulkData too small. Expected >= %llu, got %llu",
			(unsigned long long)ExpectedSize, (unsigned long long)BulkSize);

		Mip.BulkData.Unlock();
		return false;
	}

	const size_t Step = size_t(Width) * size_t(BytesPerPixel);
	cv::Mat CvImageBGRA(Height, Width, CV_8UC4, TextureData, Step);

	// Convert to 3-channel BGR for OpenCV drawing, because it expects 1 or 3 channels, not e.g. 4.

	cv::Mat CvImageBGR;
	cv::cvtColor(CvImageBGRA, CvImageBGR, cv::COLOR_BGRA2BGR);

	const bool bHasCharuco = !Corners.Corners.IsEmpty() && !Corners.CornerIds.IsEmpty();
	const bool bHasMarkers = !Corners.Markers.IsEmpty();

	if (!bHasCharuco && !bHasMarkers)
	{
		Mip.BulkData.Unlock();
		DebugTexture->UpdateResource();

		return true;
	}

	if (bHasCharuco && (Corners.Corners.Num() != Corners.CornerIds.Num()))
	{
		UE_LOGF(LogOpenCVHelper, Error, "DrawCharucoCorners: corners/ids size mismatch (%d vs %d)",
			Corners.Corners.Num(), Corners.CornerIds.Num());

		Mip.BulkData.Unlock();
		return false;
	}

	std::vector<cv::Point2f> CharucoCorners;
	std::vector<int> CharucoIds;

	if (bHasCharuco)
	{
		CharucoCorners.reserve(Corners.Corners.Num());
		for (const FVector2f& Corner : Corners.Corners)
		{
			CharucoCorners.emplace_back(Corner.X, Corner.Y);
		}

		CharucoIds.reserve(Corners.CornerIds.Num());
		for (int32 CornerId : Corners.CornerIds)
		{
			CharucoIds.push_back((int)CornerId);
		}
	}

	std::vector<std::vector<cv::Point2f>> MarkerCornerQuads;
	std::vector<int> MarkerIds;

	if (bHasMarkers)
	{
		MarkerCornerQuads.reserve(Corners.Markers.Num());
		MarkerIds.reserve(Corners.Markers.Num());

		for (const FArucoMarker& ArucoMarker : Corners.Markers)
		{
			std::vector<cv::Point2f> CornerQuad;

			CornerQuad.reserve(4);

			constexpr int32 CornersPerAruco = 4;

			for (int32 CornerIdx = 0; CornerIdx < CornersPerAruco; ++CornerIdx)
			{
				const FVector2f& Corner = ArucoMarker.Corners[CornerIdx];
				CornerQuad.emplace_back(Corner.X, Corner.Y);
			}

			MarkerCornerQuads.push_back(std::move(CornerQuad));
			MarkerIds.push_back((int)ArucoMarker.MarkerID);
		}
	}

	if (bHasCharuco)
	{
		// Draw corners as simple circles without IDs
		for (size_t CornerIdx = 0; CornerIdx < CharucoCorners.size(); ++CornerIdx)
		{
			cv::circle(CvImageBGR, CharucoCorners[CornerIdx], 5, cv::Scalar(0, 255, 0), -1);
		}
	}
	if (bHasMarkers)
	{
		// Draw marker corners without IDs
		for (const std::vector<cv::Point2f>& MarkerCornerQuad : MarkerCornerQuads)
		{
			for (size_t CornerQuadIdx = 0; CornerQuadIdx < MarkerCornerQuad.size(); ++CornerQuadIdx)
			{
				cv::line(CvImageBGR, MarkerCornerQuad[CornerQuadIdx], MarkerCornerQuad[(CornerQuadIdx + 1) % MarkerCornerQuad.size()], cv::Scalar(255, 0, 255), 2);
			}
		}
	}

	// Convert back to BGRA for Unreal texture (preserve alpha channel by expanding BGR)
	cv::cvtColor(CvImageBGR, CvImageBGRA, cv::COLOR_BGR2BGRA);

	Mip.BulkData.Unlock();
	DebugTexture->UpdateResource();

	return true;

#else

	UE_LOGF(LogOpenCVHelper, Error, "DrawCharucoCorners: OpenCV not available (WITH_OPENCV=0)");
	return false;

#endif
}


UTexture2D* FOpenCVHelper::GenerateCharucoBoard(const FCharucoBoardConfig& BoardConfig, FIntPoint ImageSize, int32 MarginSize)
{
#if WITH_OPENCV

	// Validate input parameters
	if (ImageSize.X <= 0 || ImageSize.Y <= 0)
	{
		UE_LOGF(LogOpenCVHelper, Error, "GenerateCharucoBoard: Invalid image dimensions (%d x %d). Must be positive.", 
			ImageSize.X, ImageSize.Y);
		return nullptr;
	}

	if (BoardConfig.SquaresX <= 0 || BoardConfig.SquaresY <= 0)
	{
		UE_LOGF(LogOpenCVHelper, Error, "GenerateCharucoBoard: Invalid board dimensions (%d x %d squares). Must be positive.", 
			BoardConfig.SquaresX, BoardConfig.SquaresY);
		return nullptr;
	}

	if (BoardConfig.SquareSize <= 0.0f || BoardConfig.MarkerSize <= 0.0f)
	{
		UE_LOGF(LogOpenCVHelper, Error, "GenerateCharucoBoard: Invalid sizes (SquareSize: %f, MarkerSize: %f). Must be positive.", 
			BoardConfig.SquareSize, BoardConfig.MarkerSize);
		return nullptr;
	}

	if (BoardConfig.MarkerSize >= BoardConfig.SquareSize)
	{
		UE_LOGF(LogOpenCVHelper, Error, "GenerateCharucoBoard: MarkerSize (%f) must be smaller than SquareSize (%f).", 
			BoardConfig.MarkerSize, BoardConfig.SquareSize);
		return nullptr;
	}

	if (MarginSize < 0)
	{
		UE_LOGF(LogOpenCVHelper, Error, "GenerateCharucoBoard: MarginSize (%d) cannot be negative.", MarginSize);
		return nullptr;
	}

	// Get ArUco dictionary

	cv::Ptr<cv::aruco::Dictionary> Dictionary = UE::Aruco::Private::GetArucoDictionary(BoardConfig.Dictionary);

	if (Dictionary == nullptr)
	{
		UE_LOGF(LogOpenCVHelper, Error, "GenerateCharucoBoard: Invalid ArUco dictionary");
		return nullptr;
	}

	// Create Charuco board

	cv::Ptr<cv::aruco::CharucoBoard> CharucoBoard = cv::aruco::CharucoBoard::create(
		BoardConfig.SquaresX, 
		BoardConfig.SquaresY, 
		BoardConfig.SquareSize, 
		BoardConfig.MarkerSize, 
		Dictionary
	);

	// Generate board image

	cv::Mat BoardImage;
	CharucoBoard->draw(cv::Size(ImageSize.X, ImageSize.Y), BoardImage, MarginSize, 1);

	// Convert to BGRA

	cv::Mat BoardImageBGRA;
	cv::cvtColor(BoardImage, BoardImageBGRA, cv::COLOR_GRAY2BGRA);

	// Create UE texture

	UTexture2D* NewTexture = UTexture2D::CreateTransient(ImageSize.X, ImageSize.Y, PF_B8G8R8A8);

	if (!NewTexture)
	{
		UE_LOGF(LogOpenCVHelper, Error, "GenerateCharucoBoard: Failed to create texture");
		return nullptr;
	}

	// Copy data to texture

	FTexture2DMipMap& Mip = NewTexture->GetPlatformData()->Mips[0];

	uint8* TextureData = static_cast<uint8*>(Mip.BulkData.Lock(LOCK_READ_WRITE));

	ON_SCOPE_EXIT
	{
		if (Mip.BulkData.IsLocked())
		{
			Mip.BulkData.Unlock();
		}
	};

	if (!TextureData)
	{
		UE_LOGF(LogOpenCVHelper, Error, "GenerateCharucoBoard: Failed to lock texture bulk data");

		NewTexture->MarkAsGarbage();
		return nullptr;
	}
	
	const SIZE_T BufferSize = Mip.BulkData.GetBulkDataSize();
	if (!SafeCopyMatToBuffer(BoardImageBGRA, TextureData, BufferSize))
	{
		NewTexture->MarkAsGarbage();
		return nullptr;
	}

	Mip.BulkData.Unlock();
	
	NewTexture->UpdateResource();

	UE_LOGF(LogOpenCVHelper, Log, "GenerateCharucoBoard: Generated %dx%d Charuco board with %dx%d squares", 
		ImageSize.X, ImageSize.Y, BoardConfig.SquaresX, BoardConfig.SquaresY);

	return NewTexture;
#else
	UE_LOGF(LogOpenCVHelper, Error, "GenerateCharucoBoard: OpenCV not available");
	return nullptr;
#endif
}
