// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOpenCVHelperModule.h"
#include "Modules/ModuleManager.h" // for IMPLEMENT_MODULE()
#include "Interfaces/IPluginManager.h"

#if WITH_OPENCV
// IWYU pragma: begin_keep
#include "OpenCVHelper.h"
#include "PreOpenCVHeaders.h"
#include "opencv2/unreal.hpp"
#include "PostOpenCVHeaders.h"
// IWYU pragma: end_keep
#endif

namespace OpenCVHelperModule
{
#if WITH_OPENCV

	static TSet<void*> UnrealAllocationsSet;
	static FCriticalSection UnrealAllocationsSetMutex;

	static void* UnrealMalloc(size_t Count, uint32_t Alignment)
	{
		void* Address = FMemory::Malloc(static_cast<SIZE_T>(Count), static_cast<uint32>(Alignment));

		{
			FScopeLock Lock(&UnrealAllocationsSetMutex);
			UnrealAllocationsSet.Add(Address);
		}

		return Address;
	}

	static void UnrealFree(void* Original)
	{
		// Only free allocations made by Unreal. Any allocations made before new/delete was overridden will have to leak.
		{
			FScopeLock Lock(&UnrealAllocationsSetMutex);

			if (!UnrealAllocationsSet.Contains(Original))
			{
				return;
			}

			UnrealAllocationsSet.Remove(Original);
		}

		FMemory::Free(Original);
	}

#endif //WITH_OPENCV
}

class FOpenCVHelperModule : public IOpenCVHelperModule
{
public:
	FOpenCVHelperModule();

public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void* OpenCvDllHandle;
};

FOpenCVHelperModule::FOpenCVHelperModule()
	: OpenCvDllHandle(nullptr)
{}

void FOpenCVHelperModule::StartupModule()
{
	const FString PluginDir = IPluginManager::Get().FindPlugin(TEXT("OpenCV"))->GetBaseDir();

#if WITH_OPENCV

#if defined(OPENCV_DLL_NAME)
	const FString OpenCvBinPath = PluginDir / TEXT(UE_STRINGIZE(OPENCV_PLATFORM_PATH));
	const FString DLLPath = OpenCvBinPath / TEXT(UE_STRINGIZE(OPENCV_DLL_NAME));

	FPlatformProcess::PushDllDirectory(*OpenCvBinPath);
	OpenCvDllHandle = FPlatformProcess::GetDllHandle(*DLLPath);
	FPlatformProcess::PopDllDirectory(*OpenCvBinPath);

	if (OpenCvDllHandle == nullptr)
	{
		return;
	}
#endif

	// We need to tell OpenCV to use Unreal's memory allocator to avoid crashes.
	// These may happen when Unreal passes a container to OpenCV, then OpenCV allocates memory for that container
	// and then Unreal tries to release the memory in it.
	cv::unreal::SetMallocAndFree(&OpenCVHelperModule::UnrealMalloc, &OpenCVHelperModule::UnrealFree);

#endif
}

void FOpenCVHelperModule::ShutdownModule()
{
#if WITH_OPENCV
	if (OpenCvDllHandle)
	{
		FPlatformProcess::FreeDllHandle(OpenCvDllHandle);
		OpenCvDllHandle = nullptr;
	}

	// Note: Seems safer to not put back the original new/delete in OpenCV and keep Unreal's versions even after this module unloads.

#endif
}

IMPLEMENT_MODULE(FOpenCVHelperModule, OpenCVHelper);
