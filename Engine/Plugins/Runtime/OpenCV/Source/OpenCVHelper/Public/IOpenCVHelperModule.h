// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * 
 */
class IOpenCVHelperModule : public IModuleInterface
{
public:
	/** Virtual destructor. */
	virtual ~IOpenCVHelperModule() = default;
};

