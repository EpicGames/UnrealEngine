// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "OpenCVHelper.h"

#include "OpenCVBlueprintFunctionLibrary.generated.h"

#define UE_API OPENCVHELPER_API

DECLARE_LOG_CATEGORY_EXTERN(LogOpenCVBlueprintFunctionLibrary, Log, All);

class UTexture2D;
class UTextureRenderTarget2D;

UENUM(BlueprintType)
enum class EOpenCVArucoDictionary : uint8
{
	Dict4x4 UMETA(DisplayName = "4x4"),
	Dict5x5 UMETA(DisplayName = "5x5"),
	Dict6x6 UMETA(DisplayName = "6x6"),
	Dict7x7 UMETA(DisplayName = "7x7"),
	DictOriginal UMETA(DisplayName = "Original")
};

UENUM(BlueprintType)
enum class EOpenCVArucoDictionarySize : uint8
{
	DictSize50 UMETA(DisplayName = "50"),
	DictSize100 UMETA(DisplayName = "100"),
	DictSize250 UMETA(DisplayName = "250"),
	DictSize1000 UMETA(DisplayName = "1000")
};

USTRUCT(BlueprintType)
struct FOpenCVArucoDetectedMarker
{
	GENERATED_BODY()

	FOpenCVArucoDetectedMarker()
		: Id(-1)
	{
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenCV | ArUco")
	int32 Id;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenCV | ArUco")
	TArray<FVector2D> Corners;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OpenCV | ArUco")
	FTransform Pose;
};

UCLASS(MinimalAPI, BlueprintType)
class UOpenCVBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Detects a camera calibration chessboard in the supplied image
	 *
	 * @param InRenderTarget Input image in which to search for a chessboard
	 * @param InPatternSize Number of interior corners on the physical chessboard (rows, columns)
	 * @param bDebugDrawCorners If true, output a Texture2D showing the detected corner debug info overlaid on the input image
	 * @param OutDebugTexture Output debug image (required if bDebugDrawCorners is True)
	 * @param OutDetectedCorners Output array of corners detected in the input image
	 * @return Total number of corners detected in the input image
	 */
	UFUNCTION(BlueprintCallable, Category = "OpenCV", Meta = (DisplayName = "OpenCV Chessboard Detect Corners"))
	static UE_API int32 OpenCVChessboardDetectCorners(const UTextureRenderTarget2D* InRenderTarget, const FIntPoint InPatternSize, const bool bDebugDrawCorners, UTexture2D*& OutDebugTexture, TArray<FVector2D>& OutDetectedCorners);

	/**
	 * Detects all ArUco markers in the supplied image
	 *
	 * @param InRenderTarget Input image in which to search for markers
	 * @param InDictionary Which ArUco marker dictionary to use for detection
	 * @param InDictionarySize The size of the ArUco marker dictionary
	 * @param bDebugDrawMarkers If true, output a Texture2D showing the detected marker debug info overlaid on the input image
	 * @param bEstimatePose If true, return the 3D pose for each marker relative to the camera position
	 * @param InMarkerLengthInMeters Length in meters of one side of the physical marker (required if bEstimatePose is True)
	 * @param InCameraCalibrationParameters Lens distortion parameters for the incoming image (required if bEstimatePose is True)
	 * @param OutDebugTexture Output debug image (required if bDebugDrawMarkers is True)
	 * @param OutDetectedMarkers Output array of markers detected in the input image
	 * @return Total number of markers detected in the input image
	 */
	UFUNCTION(BlueprintCallable, Category = "OpenCV | ArUco", Meta = (DisplayName = "OpenCV ArUco Detect Markers"))
	static UE_API int32 OpenCVArucoDetectMarkers(const UTextureRenderTarget2D* InRenderTarget, const EOpenCVArucoDictionary InDictionary, const EOpenCVArucoDictionarySize InDictionarySize, const bool bDebugDrawMarkers, const bool bEstimatePose, const float InMarkerLengthInMeters, const FOpenCVLensDistortionParametersBase& InLensDistortionParameters, UTexture2D*& OutDebugTexture, TArray<FOpenCVArucoDetectedMarker>& OutDetectedMarkers);
};

#undef UE_API
