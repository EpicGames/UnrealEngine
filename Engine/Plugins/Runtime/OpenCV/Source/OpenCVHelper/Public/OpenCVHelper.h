// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include <vector>

/*
 * Like many third party headers, OpenCV headers require some care when importing.
 *
 * When including opencv headers, the includes should be wrapped like this:

	#include PreOpenCVHeaders.h

	// your opencv include directives go here...

	#include PostOpenCVHeaders.h

 * note: On Linux platform there is a typedef conflict with (u)int64
 * to use OpenCV api with these types, use the helper type OpenCVUtils::int64

 */

#include "OpenCVHelper.generated.h"

#define OPENCV_INCLUDES_START static_assert(false, "Include PreOpenCVHeaders.h instead of using this macro.");

#define OPENCV_INCLUDES_END static_assert(false, "Include PostOpenCVHeaders.h instead of using this macro");

class UTexture2D;
class FString;
class FName;

#if WITH_OPENCV

namespace cv
{
	class Mat;
	template<typename _Tp> class Point_;
	template<typename _Tp> class Point3_;

	typedef Point_<float> Point2f;
	typedef Point3_<float> Point3f;
};

#endif	// WITH_OPENCV


UENUM(BlueprintType)
enum class EArucoDictionary : uint8
{
	None                 UMETA(DisplayName = "None"),
	DICT_4X4_50          UMETA(DisplayName = "DICT_4X4_50"),
	DICT_4X4_100         UMETA(DisplayName = "DICT_4X4_100"),
	DICT_4X4_250         UMETA(DisplayName = "DICT_4X4_250"),
	DICT_4X4_1000        UMETA(DisplayName = "DICT_4X4_1000"),
	DICT_5X5_50          UMETA(DisplayName = "DICT_5X5_50"),
	DICT_5X5_100         UMETA(DisplayName = "DICT_5X5_100"),
	DICT_5X5_250         UMETA(DisplayName = "DICT_5X5_250"),
	DICT_5X5_1000        UMETA(DisplayName = "DICT_5X5_1000"),
	DICT_6X6_50          UMETA(DisplayName = "DICT_6X6_50"),
	DICT_6X6_100         UMETA(DisplayName = "DICT_6X6_100"),
	DICT_6X6_250         UMETA(DisplayName = "DICT_6X6_250"),
	DICT_6X6_1000        UMETA(DisplayName = "DICT_6X6_1000"),
	DICT_7X7_50          UMETA(DisplayName = "DICT_7X7_50"),
	DICT_7X7_100         UMETA(DisplayName = "DICT_7X7_100"),
	DICT_7X7_250         UMETA(DisplayName = "DICT_7X7_250"),
	DICT_7X7_1000        UMETA(DisplayName = "DICT_7X7_1000"),
	DICT_ARUCO_ORIGINAL  UMETA(DisplayName = "DICT_ARUCO_ORIGINAL")
};


struct FArucoMarker
{
	int32 MarkerID = 0;
	FVector2f Corners[4];
};


/** Structure representing a Charuco board configuration */
struct FCharucoBoardConfig
{
	/** Number of chessboard squares in X direction */
	int32 SquaresX = 5;
	
	/** Number of chessboard squares in Y direction */
	int32 SquaresY = 7;
	
	/** Size of each square in the board (in cm) */
	float SquareSize = 4.0f;
	
	/** Size of the ArUco markers (in cm) */
	float MarkerSize = 3.0f;
	
	/** ArUco dictionary to use for the markers. Smaller dictionaries tend to be easier to parse */
	EArucoDictionary Dictionary = EArucoDictionary::DICT_4X4_50;
};


/** Structure representing detected Charuco corners */
struct FCharucoCorners
{
	/** 2D image coordinates of detected corners */
	TArray<FVector2f> Corners;
	
	/** IDs of the detected corners (based on board layout) */
	TArray<int32> CornerIds;
	
	/** Detected ArUco markers in the Charuco board */
	TArray<FArucoMarker> Markers;
};


class FOpenCVHelper
{
public:
	/** Enumeration to specify any cartesian axis in positive or negative directions */
	enum class EAxis
	{
		X, Y, Z,
		Xn, Yn, Zn,
	};

	// These axes must match the order in which they are declared in EAxis
	inline static const TArray<FVector> UnitVectors =
	{
		{  1,  0,  0 }, //  X
		{  0,  1,  0 }, //  Y
		{  0,  0,  1 }, //  Z
		{ -1,  0,  0 }, // -X
		{  0, -1,  0 }, // -Y
		{  0,  0, -1 }, // -Z
	};

	static const FVector& UnitVectorFromAxisEnum(const EAxis Axis)
	{
		return UnitVectors[std::underlying_type_t<EAxis>(Axis)];
	};

	/** Converts in-place the coordinate system of the given FTransform by specifying the source axes in terms of the destination axes */
	static OPENCVHELPER_API void ConvertCoordinateSystem(FTransform& Transform, const EAxis DstXInSrcAxis, const EAxis DstYInSrcAxis, const EAxis DstZInSrcAxis);

	/** Converts in-place an FTransform in Unreal coordinates to OpenCV coordinates */
	static OPENCVHELPER_API void ConvertUnrealToOpenCV(FTransform& Transform);

	/** Converts in-place an FTransform in OpenCV coordinates to Unreal coordinates */
	static OPENCVHELPER_API void ConvertOpenCVToUnreal(FTransform& Transform);

	/** Converts an FVector in Unreal coordinates to OpenCV coordinates */
	static OPENCVHELPER_API FVector ConvertUnrealToOpenCV(const FVector& Transform);

	/** Converts an FTransform in OpenCV coordinates to Unreal coordinates */
	static OPENCVHELPER_API FVector ConvertOpenCVToUnreal(const FVector& Transform);

public:
#if WITH_OPENCV
	/**
	 * Creates a Texture from the given Mat, if its properties (e.g. pixel format) are supported.
	 * 
	 * @param Mat The OpenCV Mat to convert.
	 * @param PackagePath Optional path to a package to create the texture in.
	 * @param TextureName Optional name for the texture. Required if PackagePath is not nullptr.
	 * 
	 * @return Texture created out of the given OpenCV Mat.
	 */
	static OPENCVHELPER_API UTexture2D* TextureFromCvMat(cv::Mat& Mat, const FString* PackagePath = nullptr, const FName* TextureName = nullptr);
	static OPENCVHELPER_API UTexture2D* TextureFromCvMat(cv::Mat& Mat, UTexture2D* InTexture);

	/** 
	 * Takes a rotation vector (in rodrigues form) and translation vector that represent a change of basis from object space to camera space (in OpenCV's coordinate system),
	 * inverts them to generate a camera pose, and converts it to an FTransform in UE's coordinate system. 
	 * The input rotation and translation vectors are expected to match the form of an "rvec" and "tvec" that are generated functions like cv::calibrateCamera() and cv::solvePnP()
	 */
	static OPENCVHELPER_API void MakeCameraPoseFromObjectVectors(const cv::Mat& InRotation, const cv::Mat& InTranslation, FTransform& OutTransform);

	/**
	 * Takes a camera pose transform in UE's coordinate system, converts it to a rotation vector (in rodrigues form) and translation vector (in OpenCV's coordinate system),
	 * and inverts them to generate vectors that represent a change of basis from object space to camera space.
	 * The output rotation and translation vectors are expected to match the form of an "rvec" and "tvec" that are consumed by functions like cv::calibrateCamera() and cv::solvePnP()
	 */
	static OPENCVHELPER_API void MakeObjectVectorsFromCameraPose(const FTransform& InTransform, cv::Mat& OutRotation, cv::Mat& OutTranslation);
#endif	// WITH_OPENCV

	/** Identify a set of aruco markers in the input image that belong to the input aruco dictionary, and output the marker IDs and the 2D coordinates of the 4 corners of each marker */
	static OPENCVHELPER_API bool IdentifyArucoMarkers(TArray<FColor>& Image, FIntPoint ImageSize, EArucoDictionary DictionaryName, TArray<FArucoMarker>& OutMarkers);

	/** Draw a debug view of the input aruco markers on top of the input texture */
	static OPENCVHELPER_API bool DrawArucoMarkers(const TArray<FArucoMarker>& Markers, UTexture2D* DebugTexture);

	/** Identify a checkerboard pattern in the input image with the given checkerboard dimensions (columns x rows), and output the 2D coordinates of the intersections between each checkerboard square */
	static OPENCVHELPER_API bool IdentifyCheckerboard(TArray<FColor>& Image, FIntPoint ImageSize, FIntPoint CheckerboardDimensions, TArray<FVector2f>& OutCorners);

	/** Identify a checkerboard pattern in a region of interest of the input image with the given checkerboard dimensions (columns x rows), and output the 2D coordinates of the intersections between each checkerboard square */
	static OPENCVHELPER_API bool IdentifyCheckerboard(TArray<FColor>& Image, FIntPoint ImageSize, FIntRect RegionOfInterest, FIntPoint CheckerboardDimensions, TArray<FVector2f>& OutCorners);

	/** Draw a debug view of the input checkerboard corners on top of the input texture */
	static OPENCVHELPER_API bool DrawCheckerboardCorners(const TArray<FVector2f>& Corners, FIntPoint CheckerboardDimensions, UTexture2D* DebugTexture);

	/** Draw a debug view of the input checkerboard corners on top of the input texture */
	static OPENCVHELPER_API bool DrawCheckerboardCorners(const TArray<FVector2D>& Corners, FIntPoint CheckerboardDimensions, UTexture2D* DebugTexture);

	/** Compute the camera pose that minimizes the reprojection error of the input object points and image points */
	static OPENCVHELPER_API bool SolvePnP(const TArray<FVector>& ObjectPoints, const TArray<FVector2f>& ImagePoints, const FVector2D& FocalLength, const FVector2D& ImageCenter, const TArray<float>& DistortionParameters, FTransform& OutCameraPose);

	/** Project the input object points to the 2D image plane defined by the input camera intrinsics and camera pose */
	static OPENCVHELPER_API bool ProjectPoints(const TArray<FVector>& ObjectPoints, const FVector2D& FocalLength, const FVector2D& ImageCenter, const TArray<float>& DistortionParameters, const FTransform& CameraPose, TArray<FVector2f>& OutImagePoints);

	/** Project the input object points to the 2D image plane defined by the input camera intrinsics and camera pose */
	static OPENCVHELPER_API bool ProjectPoints(const TArray<FVector>& ObjectPoints, const FVector2D& FocalLength, const FVector2D& ImageCenter, const TArray<float>& DistortionParameters, const FTransform& CameraPose, TArray<FVector2D>& OutImagePoints);

	/** Find a 3D fit line that passes through the input points, as well as a point on that line */
	static OPENCVHELPER_API bool FitLine3D(const TArray<FVector>& InPoints, FVector& OutLine, FVector& OutPointOnLine);

#if WITH_OPENCV
	UE_DEPRECATED(5.4, "The version of ComputeReprojectionError takes OpenCV types as input parameters is deprecated. Please use the version that takes all UE types.")
	static OPENCVHELPER_API double ComputeReprojectionError(const FTransform& CameraPose, const cv::Mat& CameraIntrinsicMatrix, const std::vector<cv::Point3f>& Points3d, const std::vector<cv::Point2f>& Points2d);
#endif

	/** Project the 3D objects points to the 2D image plane represented by the input camera intrinsics and pose, and compute the reprojection error (euclidean distance) between the input image points and the reprojected points */
	static OPENCVHELPER_API double ComputeReprojectionError(const TArray<FVector>& ObjectPoints, const TArray<FVector2f>& ImagePoints, const FVector2D& FocalLength, const FVector2D& ImageCenter, const FTransform& CameraPose);

	/** Identify Charuco corners in the input image with the given board configuration, and output the detected corners and their IDs */
	static OPENCVHELPER_API bool IdentifyCharucoCorners(TArray<FColor>& Image, FIntPoint ImageSize, const FCharucoBoardConfig& BoardConfig, FCharucoCorners& OutCorners, const FIntRect& ROI = FIntRect());

	/** Draw a debug view of the input Charuco corners on top of the input texture */
	static OPENCVHELPER_API bool DrawCharucoCorners(const FCharucoCorners& Corners, UTexture2D* DebugTexture);

	/** Generate a Charuco board image with the given configuration for display or printing */
	static OPENCVHELPER_API UTexture2D* GenerateCharucoBoard(const FCharucoBoardConfig& BoardConfig, FIntPoint ImageSize, int32 MarginSize = 10);
};

/**
 * Mathematic camera model for lens distortion/undistortion.
 * Camera matrix =
 *  | F.X  0  C.x |
 *  |  0  F.Y C.Y |
 *  |  0   0   1  |
 * where F and C are normalized.
 */
USTRUCT(BlueprintType)
struct FOpenCVLensDistortionParametersBase
{
	GENERATED_USTRUCT_BODY()

public:
	FOpenCVLensDistortionParametersBase()
		: K1(0.f)
		, K2(0.f)
		, P1(0.f)
		, P2(0.f)
		, K3(0.f)
		, K4(0.f)
		, K5(0.f)
		, K6(0.f)
		, F(FVector2D(1.f, 1.f))
		, C(FVector2D(0.5f, 0.5f))
		, bUseFisheyeModel(false)
	{
	}

public:
#if WITH_OPENCV
	/** Convert internal coefficients to OpenCV matrix representation */
	OPENCVHELPER_API cv::Mat ConvertToOpenCVDistortionCoefficients() const;

	/** Convert internal normalized camera matrix to OpenCV pixel scaled matrix representation. */
	OPENCVHELPER_API cv::Mat CreateOpenCVCameraMatrix(const FVector2D& InImageSize) const;
#endif //WITH_OPENCV

public:
	/** Compare two lens distortion models and return whether they are equal. */
	bool operator == (const FOpenCVLensDistortionParametersBase& Other) const
	{
		return (K1 == Other.K1 &&
			K2 == Other.K2 &&
			P1 == Other.P1 &&
			P2 == Other.P2 &&
			K3 == Other.K3 &&
			K4 == Other.K4 &&
			K5 == Other.K5 &&
			K6 == Other.K6 &&
			F == Other.F &&
			C == Other.C &&
			bUseFisheyeModel == Other.bUseFisheyeModel);
	}

	/** Compare two lens distortion models and return whether they are different. */
	bool operator != (const FOpenCVLensDistortionParametersBase& Other) const
	{
		return !(*this == Other);
	}

	/** Returns true if lens distortion parameters are for identity lens (or default parameters) */
	bool IsIdentity() const
	{
		return (K1 == 0.0f &&
			K2 == 0.0f &&
			P1 == 0.0f &&
			P2 == 0.0f &&
			K3 == 0.0f &&
			K4 == 0.0f &&
			K5 == 0.0f &&
			K6 == 0.0f &&
			F == FVector2D(1.0f, 1.0f) &&
			C == FVector2D(0.5f, 0.5f));
	}

	bool IsSet() const
	{
		return *this != FOpenCVLensDistortionParametersBase();
	}

public:
	/** Radial parameter #1. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	float K1;

	/** Radial parameter #2. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	float K2;

	/** Tangential parameter #1. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	float P1;

	/** Tangential parameter #2. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	float P2;

	/** Radial parameter #3. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	float K3;

	/** Radial parameter #4. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	float K4;

	/** Radial parameter #5. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	float K5;
	
	/** Radial parameter #6. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	float K6;

	/** Camera matrix's normalized Fx and Fy. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	FVector2D F;

	/** Camera matrix's normalized Cx and Cy. */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	FVector2D C;

	/** Camera lens needs Fisheye camera model. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens Distortion|Parameters")
	bool bUseFisheyeModel;
};
