// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace OpenCVUtils
{
    //Keep track of UE's int64 and uint64 typedefs 
    //to be able to restore it after including CV headers
    using UEInt64 = FPlatformTypes::int64;
    using UEUInt64 = FPlatformTypes::uint64;
};

