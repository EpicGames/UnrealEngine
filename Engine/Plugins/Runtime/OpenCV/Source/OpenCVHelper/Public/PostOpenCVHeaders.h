// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef OPENCV_HEADERS_TYPES_GUARD
#undef OPENCV_HEADERS_TYPES_GUARD
#else
#error Mismatched PreOpenCVHeadersTypes.h detected.
#endif

// HEADER_UNIT_SKIP - Special include

#include "OpenCVUtils.h"

#if PLATFORM_WINDOWS

__pragma(warning(pop))
UE_POP_MACRO("check")
THIRD_PARTY_INCLUDES_END

#elif PLATFORM_LINUX

//Stamp CV's int64 type to use it with their api and restore our definition of int64 after including CV's headers
namespace OpenCVUtils
{
    using cvint64 = int64;
    using cvuintt64 = uint64;
}

#undef int64
#undef uint64
using int64 = OpenCVUtils::UEInt64;
using uint64 = OpenCVUtils::UEUInt64;

#pragma warning(pop)
UE_POP_MACRO("check")
THIRD_PARTY_INCLUDES_END

#else

UE_POP_MACRO("check")
THIRD_PARTY_INCLUDES_END

#endif