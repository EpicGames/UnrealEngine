// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef OPENCV_HEADERS_TYPES_GUARD
#define OPENCV_HEADERS_TYPES_GUARD
#else
#error Nesting PostOpenCVHeadersTypes.h is not allowed!
#endif

#include "OpenCVUtils.h"

#if PLATFORM_WINDOWS

THIRD_PARTY_INCLUDES_START
UE_PUSH_MACRO("check")
#undef check

__pragma(warning(push))
__pragma(warning(disable: 4190))  /* 'identifier1' has C-linkage specified, but returns UDT 'identifier2' which is incompatible with C */
__pragma(warning(disable: 6297))  /* Arithmetic overflow:  32-bit value is shifted, then cast to 64-bit value.  Results might not be an expected value. */
__pragma(warning(disable: 6294))  /* Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed. */
__pragma(warning(disable: 6201))  /* Index '<x>' is out of valid index range '<a>' to '<b>' for possibly stack allocated buffer '<variable>'. */
__pragma(warning(disable: 6269))  /* Possibly incorrect order of operations:  dereference ignored. */
__pragma(warning(disable: 4263)) /* cv::detail::BlocksCompensator::feed member function does not override any base class virtual member function */
__pragma(warning(disable: 4264)) /* cv::detail::ExposureCompensator::feed : no override available for virtual member function from base 'cv::detail::ExposureCompensator'; function is hidden */

#elif PLATFORM_LINUX

THIRD_PARTY_INCLUDES_START
UE_PUSH_MACRO("check")
#undef check

#pragma warning(push) 
#pragma warning(disable: 4190)  /* 'identifier1' has C-linkage specified, but returns UDT 'identifier2' which is incompatible with C */ 
#pragma warning(disable: 6297)  /* Arithmetic overflow:  32-bit value is shifted, then cast to 64-bit value.  Results might not be an expected value. */ 
#pragma warning(disable: 6294)  /* Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed. */ 
#pragma warning(disable: 6201)  /* Index '<x>' is out of valid index range '<a>' to '<b>' for possibly stack allocated buffer '<variable>'. */ 
#pragma warning(disable: 6269)  /* Possibly incorrect order of operations:  dereference ignored. */ 
#pragma warning(disable: 4263) /* cv::detail::BlocksCompensator::feed member function does not override any base class virtual member function */ 
#pragma warning(disable: 4264) /* cv::detail::ExposureCompensator::feed : no override available for virtual member function from base 'cv::detail::ExposureCompensator'; function is hidden */ 

//Conflicting typedef of int64 and uint64 between OpenCV and UE
//Trick it by defining int64 to its own type, include CV headers 
//and put back our own type in it afterwards in PostIncludes
//Issue has been flagged in opencv github and referenced here 
//https://github.com/opencv/opencv/issues/7573
#define int64 cvint64
#define uint64 cvuint64

#else

// TODO: when adding support for other platforms, this definition may require updating
THIRD_PARTY_INCLUDES_START
UE_PUSH_MACRO("check")
#undef check

#endif



